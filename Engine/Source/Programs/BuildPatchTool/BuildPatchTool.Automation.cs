// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using System.IO;
using AutomationTool;

class BuildPatchToolBuild : BuildCommand
{
	#region Fields
	#endregion

	public override void ExecuteBuild()
	{
	}
}
