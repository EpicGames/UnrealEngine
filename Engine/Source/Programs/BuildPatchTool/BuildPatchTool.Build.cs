// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class BuildPatchTool : ModuleRules
{
	public BuildPatchTool(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePaths.Add("Runtime/Launch/Public");

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"BuildPatchServices",
				"Projects",
				"Networking",
				"HTTP",
			}
		);
		
		PublicDefinitions.Add("WITH_GOOGLE_MOCK=0");
		PublicDefinitions.Add("WITH_GOOGLE_TEST=0");
		PublicDefinitions.Add("WITH_ONLINEMCP=0");
	}
}
