// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms("Win64", "Mac", "Linux")]
[SupportedConfigurations(UnrealTargetConfiguration.Debug, UnrealTargetConfiguration.Development, UnrealTargetConfiguration.Shipping)]
public class BuildPatchToolTarget : TargetRules
{
	public BuildPatchToolTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		DefaultBuildSettings = BuildSettingsVersion.Latest;
		LinkType = TargetLinkType.Monolithic;

		LaunchModuleName = "BuildPatchTool";
		bLegalToDistributeBinary = true;
		UndecoratedConfiguration = UnrealTargetConfiguration.Shipping;

		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;
		bCompileAgainstApplicationCore = false;
		bBuildDeveloperTools = false;
		bUseLoggingInShipping = true;
		bUseChecksInShipping = true;
		bIsBuildingConsoleApplication = true;
		bHasExports = false;
		bWithServerCode = false;
		bForceDisableAutomationTests = true;

		GlobalDefinitions.Add("DISABLE_CWD_CHANGES=1");
		GlobalDefinitions.Add("UE_DONT_USE_DEFAULT_OUTPUT_DEVICES=1");
		
		// We don't use Crash Reporter right now.
		GlobalDefinitions.Add("NOINITCRASHREPORTER=1");

		// This is needed to avoid adding a dependency on thirdparty dynamic libraries that BPT doesn't use
		GlobalDefinitions.Add("DISABLE_AUDIO_MCP_MODULES=1");

		// Add definitions required by our windows RC file.
		GlobalDefinitions.Add(string.Format("PORTAL_CHANGELIST={0}", Target.Version.Changelist));
		GlobalDefinitions.Add(string.Format("PORTAL_BRANCH_NAME=\"{0}\"", Target.Version.BranchName));
	}
}
