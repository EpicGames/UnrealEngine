// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildPatchTool.h"

#include "RequiredProgramMainCPPInclude.h"

#include "Misc/App.h"
#include "Misc/OutputDeviceError.h"
#include "Http.h"

#if PLATFORM_UNIX || PLATFORM_MAC
#include <unistd.h>
#endif

#include "UObject/UObjectBase.h"

#include "BuildPatchToolVersion.h"
#include "Common/CmdLineProcessor.h"
#include "Common/VerboseLogging.h"
#include "Interfaces/IBuildPatchServicesModule.h"
#include "Interfaces/ToolMode.h"
#include "ToolModes/AutomationMode.h"
#include "ToolModes/ChunkDeltaOptimiseMode.h"
#include "ToolModes/ChunkDirectoryMode.h"
#include "ToolModes/CompactifyMode.h"
#include "ToolModes/DiffManifestMode.h"
#include "ToolModes/ExtractMetadataMode.h"
#include "ToolModes/EnumerationMode.h"
#include "ToolModes/InstallManifestMode.h"
#include "ToolModes/MergeManifestMode.h"
#include "ToolModes/PackageChunksMode.h"
#include "ToolModes/ToolModesHelp.h"
#include "ToolModes/VerifyChunksMode.h"

#if WITH_GOOGLE_MOCK
THIRD_PARTY_INCLUDES_START
#include "gmock/gmock.h"
THIRD_PARTY_INCLUDES_END
#endif

using namespace BuildPatchTool;

IMPLEMENT_APPLICATION(BuildPatchTool, "BuildPatchTool");
DEFINE_LOG_CATEGORY(LogBuildPatchTool);

namespace
{
	void SetCurrentWorkingDirectoryToBaseDir()
	{
		FPlatformMisc::CacheLaunchDir();
#if PLATFORM_WINDOWS
		TCHAR SystemError[1024];
		verifyf(::SetCurrentDirectoryW(FWindowsPlatformProcess::BaseDir()), TEXT("Failed to set the working directory to '%s' (%s)"),
			FWindowsPlatformProcess::BaseDir(),
			FWindowsPlatformMisc::GetSystemErrorMessage(SystemError, UE_ARRAY_COUNT(SystemError), 0));
#elif PLATFORM_UNIX || PLATFORM_MAC
		// TODO why not just call FPlatformMisc::SetCurrentWorkingDirectory(*FPlatformProcess::BaseDir());
		// which would remove all this platform specific code.  Should be no need for this, or? -jwf
		FTCHARToUTF8 Converted(FPlatformProcess::BaseDir());
		chdir(Converted.Get());
#else
#error Please implement SetCurrentWorkingDirectoryToBaseDir() for current platform
#endif
	}
}

namespace CommandLineHelpers
{
	bool ParseSwitch(const TCHAR* InSwitch, FString& Value, const TArray<FString>& Switches)
	{
		// Debug check requirements for InSwitch
		checkSlow(InSwitch != nullptr);
		checkSlow(InSwitch[FCString::Strlen(InSwitch) - 1] == TEXT('='));

		for (const FString& Switch : Switches)
		{
			if (Switch.StartsWith(InSwitch))
			{
				FString StringValue;
				Switch.Split(TEXT("="), nullptr, &StringValue);
				Value = StringValue.TrimQuotes();
				return true;
			}
		}
		return false;
	}
}

namespace WorkingDirectory
{
	// By default in UE4 projects the current working dir is the *.exe file location dir
	// While using BPT it is more convenient to work with files from launch directory
	// That's why BPT is built with DISABLE_CWD_CHANGES=1 define and the below code is added (EGS-23907)

	static FString LaunchDirectoryPath;

	void Configure()
	{
		LaunchDirectoryPath = FPlatformProcess::GetCurrentWorkingDirectory() + TEXT("/");
		SetCurrentWorkingDirectoryToBaseDir();
	}
}

class FBuildPatchOutputDevice : public FOutputDevice
{
public:
	FBuildPatchOutputDevice()
		: bIsInitialized(false)
	{}

	void Initialize()
	{
		GConfig->GetArray(TEXT("BuildPatchTool.ConsoleLog"), TEXT("ExcludedCategories"), ExcludedCategories, GEngineIni);
		bIsInitialized = true;
	}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override
	{
		FString LogLine;
		bool bShouldLog = false;
		if (Verbosity <= ELogVerbosity::Display)
		{
			FString CategoryStr = Category.ToString();
			// We should always log our own loggers display messages.
			bShouldLog = CategoryStr.Equals(TEXT("LogBuildPatchTool"));
			// Once we've been initialized, we should log things that we haven't explicitly excluded.
			bShouldLog |= bIsInitialized && !ExcludedCategories.Contains(CategoryStr);
			if (bShouldLog)
			{
				// We've disabled the console logger to give us finer grained control over our log output.
				// This message is something we want to print, so we should display without the log "clutter".
				LogLine = V;
			}
		}

		if (bShouldLog)
		{
			FGenericPlatformMisc::LocalPrint(*(LogLine + TEXT("\n")));
			fflush(stdout);
		}
	}

private:
	bool bIsInitialized;
	TArray<FString> ExcludedCategories;
};

const FString GetRedactedCommandLine(const TCHAR* CommandLine)
{
    FString CommandLineString;
    CommandLineString = CommandLine;
    const FString Redaction = TEXT("******");
    const TArray<FString> ParamsToRedact = { TEXT("ClientSecret") ,TEXT("EncryptionSecretKey") };
    TArray<FString> Tokens, Switches;
	FCommandLine::Parse(CommandLine, Tokens, Switches);
    for (const FString& ParamToRedact : ParamsToRedact)
    {
        FString SearchString = ParamToRedact + TEXT("=");
        FString Value;
        if (CommandLineHelpers::ParseSwitch(*SearchString, Value, Switches))
        {
            CommandLineString.ReplaceInline(*(SearchString + Value), *(SearchString + Redaction));
        }
    }

    return CommandLineString;
}

struct FModuleLoadHelper
{
public:
	FModuleLoadHelper(TCHAR const * const InModuleName)
		: ModuleName(InModuleName)
	{
		FModuleManager::Get().LoadModuleChecked(ModuleName);
	}

	~FModuleLoadHelper()
	{
		FModuleManager::Get().UnloadModule(ModuleName);
	}

	template <typename ModuleInterface>
	ModuleInterface& Get() const
	{
		return FModuleManager::GetModuleChecked<ModuleInterface>(ModuleName);
	}

	template <typename ModuleInterface>
	TSharedRef<ModuleInterface>& GetShared() const
	{
		return FModuleManager::GetModuleChecked<ModuleInterface>(ModuleName);
	}

private:
	TCHAR const * const ModuleName;
};

EReturnCode RunBuildPatchTool(const TCHAR* CommandLine)
{
	using namespace BuildPatchTool;
#if !UE_BUILD_SHIPPING
	if (FParse::Param(CommandLine, TEXT("debugging")))
	{
		while (!FPlatformMisc::IsDebuggerPresent())
		{
			FPlatformProcess::Sleep(0.01f); // to avoid 100% CPU load
		}
	}
#endif

	// Initialise the UObject module.
	FModuleLoadHelper LoadedCoreUObjectModule(TEXT("CoreUObject"));
	FCoreDelegates::OnInit.Broadcast();

	// Load the BuildPatchServices Module.
	FModuleLoadHelper LoadedBuildPatchServicesModule(TEXT("BuildPatchServices"));
	IBuildPatchServicesModule& BuildPatchServicesModule = LoadedBuildPatchServicesModule.Get<IBuildPatchServicesModule>();

	// Make sure we have processed UObjects from BPS.
	ProcessNewlyLoadedUObjects();

	// Unfortunately we cannot control UE logging going to GWarn as well as our own simpler log.
	class FFeedbackContext* GWarnBackup = GWarn;
	GWarn = nullptr;
	ON_SCOPE_EXIT{ GWarn = GWarnBackup; };

	// Run the tool mode.
	TUniquePtr<IToolMode> ToolMode(FToolModeFactory::Create(BuildPatchServicesModule, CommandLine));
	return ToolMode->Execute();
}

int32 NumberOfWorkerThreadsDesired()
{
	const int32 MaxThreads = 64;
	const int32 NumberOfCores = FPlatformMisc::NumberOfCores();
	int32 NumThreadsOverride = NumberOfCores;
	FParse::Value(FCommandLine::Get(), TEXT("NumWorkerThreads="), NumThreadsOverride);
	int32 DesiredNumberOfThreads = FMath::Max(NumberOfCores - 1, NumThreadsOverride);
	UE_LOGF(LogBuildPatchTool, Log, "Desired number of threads is [%d] based on NumberOfCores [%d] and NumThreadsOverride [%d].", DesiredNumberOfThreads, NumberOfCores, NumThreadsOverride);
	// need to spawn at least one worker thread (see FTaskGraphImplementation)
	return FMath::Max(FMath::Min(DesiredNumberOfThreads, MaxThreads), 1);
}

void CheckAndReallocThreadPool()
{
	if (FPlatformProcess::SupportsMultithreading())
	{
		const int32 ThreadsSpawned = GThreadPool->GetNumThreads();
		const int32 DesiredThreadCount = NumberOfWorkerThreadsDesired();

		if (ThreadsSpawned < DesiredThreadCount)
		{
			UE_LOGF(LogBuildPatchTool, Log, "Engine only spawned %d worker threads, bumping up to %d!", ThreadsSpawned, DesiredThreadCount);
			GThreadPool->Destroy();
			GThreadPool = FQueuedThreadPool::Allocate();
			verify(GThreadPool->Create(DesiredThreadCount, 128 * 1024));
		}
		else
		{
			UE_LOGF(LogBuildPatchTool, Log, "Continuing with %d spawned worker threads.", ThreadsSpawned);
		}
	}
}

EReturnCode BuildPatchToolMain(int32 ArgC, TCHAR* ArgV[])
{
	FTaskTagScope Scope(ETaskTag::EGameThread);
	FString EngineExitReason = TEXT("Exiting");
	ON_SCOPE_EXIT
	{
		RequestEngineExit(*EngineExitReason);
	};

	GAlwaysReportCrash = true;
	FApp::SetProjectName(TEXT("BuildPatchTool"));
	FCmdLineProcessor CmdLineProcessor(ArgC, ArgV, WorkingDirectory::LaunchDirectoryPath, [](const FString& LogLine)
	{
		FGenericPlatformMisc::LocalPrint(*(LogLine + TEXT("\n")));
		UE_LOGF(LogBuildPatchTool, Error, "%ls", *LogLine);
	});
	FCommandLine::Set(*CmdLineProcessor.GetEngineString());

	CmdLineProcessor.ProcessDragAndDrop();

	if (!CmdLineProcessor.ProcessCommandlineFiles())
	{
		EngineExitReason = TEXT("BuildPatchToolMain ProcessCommandlineFileArgs failed.");
		return EReturnCode::ArgumentProcessingError;
	}

	if (!CmdLineProcessor.HandleLegacyCommandline())
	{
		EngineExitReason = TEXT("BuildPatchToolMain HandleLegacyCommandline failed.");
		return EReturnCode::ArgumentProcessingError;
	}

	if (!CmdLineProcessor.ConvertCommandlinePaths())
	{
		EngineExitReason = TEXT("BuildPatchToolMain ConvertCommandlinePaths failed.");
		return EReturnCode::ArgumentProcessingError;
	}

	CmdLineProcessor.AddDesiredEngineFlags();

	bool bVerboseLogsSpecified = CmdLineProcessor.ContainsFlag(TEXT("-verboseLogs"));
	SetupVerboseLogging(bVerboseLogsSpecified);

	// Add log device for stdout
	bool bStdOutSpecified = CmdLineProcessor.ContainsFlag(TEXT("-stdout"));
	FBuildPatchOutputDevice* BuildPatchOutputDevice = nullptr;
	if (!bStdOutSpecified)
	{
		BuildPatchOutputDevice = new FBuildPatchOutputDevice();
		GLog->AddOutputDevice(BuildPatchOutputDevice);
	}

	// Initialise application
	const FString FullCommandLine = CmdLineProcessor.GetEngineString();
	GEngineLoop.PreInit(*FullCommandLine);
	ON_SCOPE_EXIT
	{
		FEngineLoop::AppPreExit();
		FModuleManager::Get().UnloadModulesAtShutdown();
		FEngineLoop::AppExit();
	};
	if (BuildPatchOutputDevice != nullptr)
	{
		BuildPatchOutputDevice->Initialize();
	}

	// Explicitly set the current culture to the supported value
	FInternationalization::Get().SetCurrentCulture("en");

	UE_LOGF(LogBuildPatchTool, Log, "BuildPatchTool v%ls %ls", BUILDPATCHTOOL_VERSION_STRING, LexToString(FApp::GetBuildConfiguration()));
	UE_LOGF(LogBuildPatchTool, Log, "Received  commandline: %ls", *(GetRedactedCommandLine(*CmdLineProcessor.GetOriginalCommandLine())))
	UE_LOGF(LogBuildPatchTool, Log, "Processed commandline: %ls", *(GetRedactedCommandLine(*FullCommandLine)));

	// Allow QA to test crashes are handled appropriately
	bool bForceCrashSpecified = CmdLineProcessor.ContainsFlag(TEXT("-forceCrash"));
	if (bForceCrashSpecified)
	{
		*((volatile int32*)0) = 0;
	}

	// Check whether as a program, we should bump up the number of threads in GThreadPool.
	CheckAndReallocThreadPool();

	// Run the application
	EReturnCode ReturnCode = RunBuildPatchTool(*FullCommandLine);
	if (ReturnCode != EReturnCode::OK)
	{
		UE_LOGF(LogBuildPatchTool, Error, "Tool exited with %ls (%d)", LexToString(ReturnCode), (int32)ReturnCode);
		UE_LOGF(LogBuildPatchTool, Error, "For more details, see log file at %ls", *FPaths::ConvertRelativePathToFull(FPlatformOutputDevices::GetAbsoluteLogFilename()));
	}
	return ReturnCode;
}

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{

#if WITH_GOOGLE_MOCK
	int argc = 0;
	wchar_t** argv = nullptr;
	::testing::InitGoogleMock(&argc, argv);
#endif
	
	WorkingDirectory::Configure();

	EReturnCode ReturnCode;
	// Using try&catch is the windows-specific method of interfacing with CrashReportClient
#if PLATFORM_WINDOWS && !PLATFORM_SEH_EXCEPTIONS_DISABLED
	__try
#endif
	{
		// SetCrashHandler(nullptr) sets up default behavior for Linux and Mac interfacing with CrashReportClient
		FPlatformMisc::SetCrashHandler(nullptr);
		GIsGuarded = 1;
		ReturnCode = BuildPatchToolMain(ArgC, ArgV);
		GIsGuarded = 0;
	}
#if PLATFORM_WINDOWS && !PLATFORM_SEH_EXCEPTIONS_DISABLED
	__except (ReportCrash(GetExceptionInformation()))
	{
		ReturnCode = EReturnCode::Crash;
		GError->HandleError();
	}
#endif
	return static_cast<int32>(ReturnCode);
}
