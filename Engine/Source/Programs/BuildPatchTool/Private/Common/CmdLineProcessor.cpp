// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/CmdLineProcessor.h"

#include "Algo/AllOf.h"
#include "Algo/Count.h"
#include "Algo/IndexOf.h"
#include "Algo/Replace.h"
#include "Containers/ArrayView.h"
#include "Containers/Set.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

namespace
{
	void TrimQuotePairs(FString& Argument)
	{
		static const TCHAR* Quote = TEXT("\"");
		if (Argument.StartsWith(Quote) && Argument.EndsWith(Quote))
		{
			Argument.MidInline(1, Argument.Len() - 2);
		}
	}

	void AdjustArgument(FString& Argument)
	{
		static_assert(INDEX_NONE == -1, "Logic in this function relies on INDEX_NONE == -1, if this changes, revisit.");
		static const TCHAR* Quote = TEXT("\"");
		TrimQuotePairs(Argument);
		if (Argument.Contains(TEXT(" ")))
		{
			const int32 EqualsIdx = Argument.Find(TEXT("="));
			// If = is not found, it will return -1 so that the logic will still work for expecting quote at the beginning
			// rather than after the equals.
			const int32 ExpectQuoteIdx = EqualsIdx + 1;
			const bool bAlreadyQuoted = Argument[ExpectQuoteIdx] == *Quote;
			if (!bAlreadyQuoted)
			{
				Argument += Quote;
				Argument.InsertAt(EqualsIdx + 1, Quote);
			}
		}
	}

	bool IsPureToken(const FString& Token)
	{
		// Pure tokens do not start with -.. i.e. are not flags or switches.
		// Pure token can be empty string.
		return Token.Len() == 0 || Token[0] != TEXT('-');
	}

	bool IsEmptyOrWhitespace(const TCHAR* Str)
	{
		while (*Str && FChar::IsWhitespace(*Str)) { ++Str; }
		return *Str == 0;
	}

	template <typename PredicateType>
	int32 FindByPredicate(const TArray<FString>& Tokens, int32 SearchStart, PredicateType Pred)
	{
		if (Tokens.IsValidIndex(SearchStart))
		{
			const int32 FindResult = Algo::IndexOfByPredicate(MakeArrayView(&Tokens[SearchStart], Tokens.Num() - SearchStart), Pred);
			if (FindResult != INDEX_NONE)
			{
				return FindResult + SearchStart;
			}
		}
		return INDEX_NONE;
	}

	int32 FindFlag(const TArray<FString>& Tokens, const TCHAR* Flag, int32 SearchStart = 0)
	{
		checkSlow(Flag != nullptr);
		checkSlow(Flag[0] == TEXT('-'));
		checkSlow(Flag[FCString::Strlen(Flag) - 1] != TEXT('='));
		return FindByPredicate(Tokens, SearchStart, [Flag](const FString& Elem) { return Elem == Flag; });
	}

	int32 FindSwitch(const TArray<FString>& Tokens, const TCHAR* Switch, int32 SearchStart = 0)
	{
		checkSlow(Switch != nullptr);
		checkSlow(Switch[0] == TEXT('-'));
		checkSlow(Switch[FCString::Strlen(Switch) - 1] == TEXT('='));
		return FindByPredicate(Tokens, SearchStart, [Switch](const FString& Elem) { return Elem.StartsWith(Switch); });
	}

	void ConvertRelativePathsToAbsolute(const FString& WorkingDirectory, const FString& Switch, FString Value, FString& OutReplacement)
	{
		TrimQuotePairs(Value);
		if (IsEmptyOrWhitespace(*Value))
		{
			// Leave OutReplacement unchanged.
			return;
		}
		FString ConvertedPaths;
		FString LeftS;
		FString RightS(Value);
		while (RightS.Split(TEXT(","), &LeftS, &RightS))
		{
			LeftS.TrimStartAndEndInline();
			// We don't need to convert URI, also the value in ConvertRelativePathToFull will not be changed if it is absolute.
			FString ConvertedValue = LeftS.Contains("://") ? LeftS : FPaths::ConvertRelativePathToFull(WorkingDirectory, LeftS);
			ConvertedPaths.Append(ConvertedValue);
			ConvertedPaths.AppendChar(',');
		}
		RightS.TrimStartAndEndInline();
		FString ConvertedValue = RightS.Contains("://") ? RightS : FPaths::ConvertRelativePathToFull(WorkingDirectory, RightS);
		ConvertedPaths.Append(ConvertedValue);
		OutReplacement = FString::Printf(TEXT("%s\"%s\""), *Switch, *ConvertedPaths);
	}

	void CollapseRelativeDirectories(const FString& WorkingDirectory, const FString& Switch, FString Value, FString& OutReplacement)
	{
		TrimQuotePairs(Value);
		if (IsEmptyOrWhitespace(*Value))
		{
			// Leave OutReplacement unchanged.
			return;
		}
		const bool bIsRelativePath = FPaths::IsRelative(Value);
		FPaths::NormalizeFilename(Value);
		if (FPaths::CollapseRelativeDirectories(Value))
		{
			if (bIsRelativePath)
			{
				Value.RemoveFromStart(TEXT("/"));
			}
			OutReplacement = FString::Printf(TEXT("%s\"%s\""), *Switch, *Value);
		}
	}

	template<typename TTransformFunc>
	void TransformCommandlinePaths(const FString& WorkingDirectory, const TArray<FString>& Params, TArray<FString>& Tokens, TTransformFunc Transform)
	{
		for (const FString& Param : Params)
		{
			int32 switchIdx = -1;
			while ((switchIdx = FindSwitch(Tokens, *Param, switchIdx+1)) != INDEX_NONE)
			{
				const TCHAR* Value = &Tokens[switchIdx][0];
				Value += Param.Len();
				Transform(WorkingDirectory, Param, Value, Tokens[switchIdx]);
			}
		}
	}

	bool LoadFileContentsToString(FString& Result, const TCHAR* Filename)
	{
		TUniquePtr<FArchive> Reader(IFileManager::Get().CreateFileReader(Filename));
		if (!Reader)
		{
			return false;
		}
		const int32 FileSize = Reader->TotalSize();
		if (FileSize == 0)
		{
			Result.Empty();
			return true;
		}
		TArray<uint8> FileData;
		FileData.AddUninitialized(FileSize);
		Reader->Serialize(FileData.GetData(), FileData.Num());
		const bool bSuccess = Reader->Close();
		if (bSuccess)
		{
			FFileHelper::BufferToString(Result, FileData.GetData(), FileData.Num());
		}
		return bSuccess;
	}
}

FCmdLineProcessor::FCmdLineProcessor(int32 ArgC, TCHAR* ArgV[], const FString& InWorkingDirectory, TFunction<void(const FString&)> InErrorLogger)
	: WorkingDirectory(InWorkingDirectory)
	, ErrorLogger(InErrorLogger)
{
	for (int32 Option = 1; Option < ArgC; Option++)
	{
		FString Argument(ArgV[Option]);
		AdjustArgument(Argument);
		Tokens.Add(Argument);
		if (!OriginalCommandLine.IsEmpty())
		{
			OriginalCommandLine += TEXT(" ");
		}
		OriginalCommandLine += Argument;
	}
}

bool FCmdLineProcessor::ContainsFlag(const TCHAR* Flag) const
{
	return FindFlag(Tokens, Flag) != INDEX_NONE;
}

bool FCmdLineProcessor::ContainsSwitch(const TCHAR* Switch) const
{
	return FindSwitch(Tokens, Switch) != INDEX_NONE;
}

bool FCmdLineProcessor::GetSwitchValue(const TCHAR* Switch, FString& OutValue) const
{
	const int32 SwitchIdx = FindSwitch(Tokens, Switch);
	if (SwitchIdx != INDEX_NONE)
	{
		OutValue = Tokens[SwitchIdx].RightChop(FCString::Strlen(Switch));
		return true;
	}
	return false;
}

const FString& FCmdLineProcessor::GetOriginalCommandLine()
{
	return OriginalCommandLine;
}

FString FCmdLineProcessor::GetEngineString()
{
	return FString::Join(Tokens, TEXT(" "));
}

void FCmdLineProcessor::ProcessDragAndDrop()
{
	const bool bHasDragAndDropFiles = Tokens.Num() > 0 && Algo::AllOf(Tokens, &IsPureToken);
	if (bHasDragAndDropFiles)
	{
		TArray<FString> DragAndDropTokens =
		{
			TEXT("-mode=ExtractMetadata"),
			TEXT("-outputformat=human"),
			TEXT("-openoutput")
		};

		class FFileCommandlineBuilder : public IPlatformFile::FDirectoryVisitor
		{
		public:
			FFileCommandlineBuilder(TArray<FString>& InDragAndDropTokens)
				: DragAndDropTokens(InDragAndDropTokens)
			{}

			virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
			{
				if (!bIsDirectory)
				{
					DragAndDropTokens.Add(FString::Printf(TEXT("-inputfile=\"%s\""), FilenameOrDirectory));
					DragAndDropTokens.Add(FString::Printf(TEXT("-outputfile=\"%s.txt\""), *FPaths::Combine(FPlatformProcess::UserTempDir(), FPaths::GetCleanFilename(FilenameOrDirectory))));
				}
				return true;
			}
			TArray<FString>& DragAndDropTokens;
		};

		FFileCommandlineBuilder FileCommandlineBuilder(DragAndDropTokens);

		IFileManager& FileManager = IFileManager::Get();
		for (FString DragDropFile : Tokens)
		{
			DragDropFile.TrimQuotesInline();
			if (FileManager.DirectoryExists(*DragDropFile))
			{
				FileManager.IterateDirectoryRecursively(*DragDropFile, FileCommandlineBuilder);
			}
			else
			{
				FileCommandlineBuilder.Visit(*DragDropFile, false);
			}
		}

		Tokens = MoveTemp(DragAndDropTokens);
	}
}

bool FCmdLineProcessor::ProcessCommandlineFiles()
{
	static const TCHAR* CommandLineFileSwitch = TEXT("-commandlinefile=");
	static const int32 CommandLineFileSwitchLen = FCString::Strlen(CommandLineFileSwitch);
	bool bSuccess = true;
	bool bFoundCommandlineFiles = false;
	TSet<FString> LoadedFiles;
	for (int TokenIdx = 0; TokenIdx < Tokens.Num(); TokenIdx++)
	{
		const FString& Token = Tokens[TokenIdx];
		if (Token.StartsWith(CommandLineFileSwitch))
		{
			FString CommandLineFile = Token.RightChop(CommandLineFileSwitchLen);
			TrimQuotePairs(CommandLineFile);
			const bool bCommandLineFileEmpty = CommandLineFile.IsEmpty() || Algo::AllOf(CommandLineFile, &FChar::IsWhitespace);
			if (!bCommandLineFileEmpty)
			{
				CommandLineFile = FPaths::ConvertRelativePathToFull(WorkingDirectory, CommandLineFile);

				bool bRecursive = false;
				LoadedFiles.Add(CommandLineFile, &bRecursive);
				if (bRecursive)
				{
					FString LogLine = TEXT("Aborting due to recursive reference to CommandLineFile ") + CommandLineFile;
					ErrorLogger(LogLine);
					bSuccess = false;
				}
				else
				{
					FString CommandLineFileContents;
					const bool bCommandLineFileLoadSuccess = LoadFileContentsToString(CommandLineFileContents, *CommandLineFile);
					if (bCommandLineFileLoadSuccess)
					{
						TArray<FString> NewTokens, Switches;
						CommandLineFileContents.ReplaceInline(TEXT("\r"), TEXT(" "), ESearchCase::CaseSensitive);
						CommandLineFileContents.ReplaceInline(TEXT("\n"), TEXT(" "), ESearchCase::CaseSensitive);
						const TCHAR* CommandlinePtr = *CommandLineFileContents;
						FString NewToken;
						while (FParse::Token(CommandlinePtr, NewToken, true))
						{
							AdjustArgument(NewToken);
							NewTokens.Add(MoveTemp(NewToken));
						}
						Tokens.RemoveAt(TokenIdx);
						Tokens.Insert(NewTokens, TokenIdx);
						--TokenIdx;
					}
					else
					{
						FString LogLine = TEXT("Could not load provided CommandLineFile ") + CommandLineFile;
						ErrorLogger(LogLine);
						bSuccess = false;
					}
				}
			}
			else
			{
				ErrorLogger(TEXT("CommandLineFile argument must not be empty."));
				bSuccess = false;
			}
		}
	}
	return bSuccess;
}

bool FCmdLineProcessor::HandleLegacyCommandline()
{
	auto SwitchValueEquals = [this](const TCHAR* Switch, const TCHAR* Value)
		{
			FString SwitchValue;
			return GetSwitchValue(Switch, SwitchValue) && SwitchValue == Value;
		};

	// No longer supported options.
	if (ContainsFlag(TEXT("-nochunks")))
	{
		ErrorLogger(TEXT("NoChunks is no longer a supported mode. Remove this commandline option."));
		return false;
	}

	// Check for duplicated parameters.
	if (ContainsSwitch(TEXT("-AppName=")) && ContainsSwitch(TEXT("-ArtifactId=")))
	{
		ErrorLogger(TEXT("Please don't use both `-AppName=` and `-ArtifactId=` parameters, use `-ArtifactId=` only."));
		return false;
	}

	if (ContainsFlag(TEXT("-compactify")) && ContainsSwitch(TEXT("-mode=")))
	{
		ErrorLogger(TEXT("Please don't use both `-compactify` and `-mode=` parameters, use -mode=Compactify only."));
		return false;
	}

	if (ContainsFlag(TEXT("-dataenumerate")) && ContainsSwitch(TEXT("-mode=")))
	{
		ErrorLogger(TEXT("Please don't use both `-dataenumerate` and `-mode=` parameters, use -mode=Enumeration only."));
		return false;
	}

	if (ContainsFlag(TEXT("-compactify")) && ContainsFlag(TEXT("-dataenumerate")))
	{
		ErrorLogger(TEXT("Please don't use both `-compactify` and `-dataenumerate` parameters."));
		return false;
	}

	if (Algo::CountIf(Tokens, [](const FString& Elem) { return Elem.StartsWith(TEXT("-mode=")); }) > 1)
	{
		ErrorLogger(TEXT("Please don't set more than one `-mode=` parameter."));
		return false;
	}

	if ((SwitchValueEquals(TEXT("-mode="), TEXT("enumeration")) || ContainsFlag(TEXT("-dataenumerate")))
		&& ContainsSwitch(TEXT("-ManifestFile=")) && ContainsSwitch(TEXT("-InputFile=")))
	{
		ErrorLogger(TEXT("Please don't use both `-ManifestFile=` and `-InputFile=` parameters, use `-InputFile=` only."));
		return false;
	}

	// Handle renamed params.
	const int32 AppNameIdx = FindSwitch(Tokens, TEXT("-appname="));
	if (AppNameIdx != INDEX_NONE)
	{
		Tokens[AppNameIdx].ReplaceInline(TEXT("-appname="), TEXT("-ArtifactId="));
	}

	// Handle renamed or alternative spelling modes.
	Algo::Replace(Tokens, FString("-mode=ChunkDeltaOptimize"), FString("-mode=ChunkDeltaOptimise"));

	// Check for legacy tool mode switching, if we don't have a mode and this was not a -help request, add the correct mode.
	if (!ContainsSwitch(TEXT("-mode=")) && !ContainsFlag(TEXT("-help")))
	{
		if (ContainsFlag(TEXT("-compactify")))
		{
			Algo::Replace(Tokens, FString("-compactify"), FString("-mode=compactify"));
		}
		else if (ContainsFlag(TEXT("-dataenumerate")))
		{
			Algo::Replace(Tokens, FString("-dataenumerate"), FString("-mode=enumeration"));
		}
		// Patch generation did not have a mode flag, but does have some unique and required params.
		else if (ContainsSwitch(TEXT("-BuildRoot=")) && ContainsSwitch(TEXT("-BuildVersion=")))
		{
			Tokens.Insert(TEXT("-mode=patchgeneration"), 0);
		}
	}

	// Rename parameters for single modes only.
	if (SwitchValueEquals(TEXT("-mode="), TEXT("enumeration")))
	{
		const int32 ManifestFileIdx = FindSwitch(Tokens, TEXT("-manifestfile="));
		if (ManifestFileIdx != INDEX_NONE)
		{
			Tokens[ManifestFileIdx].ReplaceInline(TEXT("-manifestfile="), TEXT("-inputfile="));
		}
	}

	// Also upgrade for any renamed offline specific modes.
	Algo::Replace(Tokens, FString("-mode=PatchGeneration"), FString("-mode=ChunkBuildDirectory"));


	// TODOBPTONLINE make commandline conversions available

	return true;
}

bool FCmdLineProcessor::ConvertCommandlinePaths()
{
	if (WorkingDirectory.IsEmpty())
	{
		return false;
	}
	TransformCommandlinePaths(WorkingDirectory, PathParamsToMakeAbsolute, Tokens, &ConvertRelativePathsToAbsolute);
	TransformCommandlinePaths(WorkingDirectory, PathParamsToCollapseRelativeDirectories, Tokens, &CollapseRelativeDirectories);
	return true;
}

void FCmdLineProcessor::AddDesiredEngineFlags()
{
	TArray<FString> EngineFlags;

	// Find and transfer all well-known engine options and move to start, for cases when commandline becomes very long.
	const int32 stdoutIdx = FindFlag(Tokens, TEXT("-stdout"));
	if (stdoutIdx != INDEX_NONE)
	{
		Tokens.RemoveAt(stdoutIdx);
		EngineFlags.Add(TEXT("-stdout"));
		EngineFlags.Add(TEXT("-FullStdOutLogOutput"));
	}

	// TODOBPTONLINE
	const int32 epicEnvIdx = FindSwitch(Tokens, TEXT("-epicenv="));
	if (epicEnvIdx != INDEX_NONE)
	{
		EngineFlags.Add(MoveTemp(Tokens[epicEnvIdx]));
		Tokens.RemoveAt(epicEnvIdx);
	}

#if UE_BUILD_DEBUG
	// Run smoke tests in debug.
	EngineFlags.Add(TEXT("-bForceSmokeTests"));
#endif

	EngineFlags.Add(TEXT("-usehyperthreading"));
	EngineFlags.Add(TEXT("-unattended"));

#if PLATFORM_MAC
	EngineFlags.Add(TEXT(" -UseNSURLSession"));
#endif

	// Insert to the beginning.
	Tokens.Insert(EngineFlags, 0);
}
