// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/DiffAbortThresholdArgument.h"

#include "Internationalization/Regex.h"
#include "BuildPatchTool.h"
#include "Misc/DefaultValueHelper.h"

using namespace BuildPatchTool;
using namespace BuildPatchServices;

namespace
{
	bool CanBeInterpretedAsPercentages(float Value, const FString& RawUnit)
	{
		return RawUnit == TEXT("PC") || RawUnit == TEXT("PCT");
	}

	TOptional<FDiffAbortThreshold> AsDiffAbortThreshold(const FString& RawValue)
	{
		// Need help with regex? Try https://regex101.com/
		const FRegexPattern Pattern(TEXT(R"(^((?:[0-9]*[.])?[0-9]+)(\s*(?:KIB|MIB|GIB|KB|MB|GB|PC|PCT))?$)"));
		FString RawValueUp = RawValue.ToUpper();
		RawValueUp.RemoveSpacesInline();
		FRegexMatcher Matcher(Pattern, RawValueUp);
		if (!Matcher.FindNext())
		{
			return {};
		}
		double Value = .0;
		if (!FDefaultValueHelper::ParseDouble(Matcher.GetCaptureGroup(1), Value))
		{
			return {};
		}
		const FString RawUnit = Matcher.GetCaptureGroup(2);
		if (CanBeInterpretedAsPercentages(Value, RawUnit))
		{
			return { { static_cast<uint64>(FMath::RoundHalfFromZero(Value)), EDiffAbortThresholdUnits::Percentage } };
		}
		const TMap<FString, uint64> Units{
			{TEXT(""), 1ULL},
			{TEXT("KIB"), 1024ULL},
			{TEXT("KB"), 1000ULL},
			{TEXT("MIB"), 1024ULL * 1024ULL},
			{TEXT("MB"), 1000ULL * 1000ULL},
			{TEXT("GIB"), 1024ULL * 1024ULL * 1024ULL},
			{TEXT("GB"), 1000ULL * 1000ULL * 1000ULL}
		};
		if (const uint64* factor = Units.Find(RawUnit))
		{
			return { { static_cast<uint64>(FMath::RoundHalfFromZero(*factor * Value)) , EDiffAbortThresholdUnits::Absolute} };
		}
		return {};
	}
}

TOptional<FDiffAbortThreshold> FDiffAbortThresholdArgument::Parse(BuildPatchTool::IToolMode& ToolMode, const TArray<FString>& Switches)
{
	FString DiffAbortThresholdRaw;
	ToolMode.ParseSwitch(TEXT("DiffAbortThreshold="), DiffAbortThresholdRaw, Switches);
	if (DiffAbortThresholdRaw.IsEmpty())
	{
		return {};
	}
	const TOptional<FDiffAbortThreshold> DiffAbortThreshold = AsDiffAbortThreshold(DiffAbortThresholdRaw);
	if (!DiffAbortThreshold)
	{
		UE_LOGF(LogBuildPatchTool, Error, "Requested -DiffAbortThreshold=%ls has invalid format.", *DiffAbortThresholdRaw);
		return {};
	}

	return ClampToSaneRange(DiffAbortThresholdRaw, DiffAbortThreshold.GetValue());
}

FDiffAbortThreshold FDiffAbortThresholdArgument::ClampToSaneRange(const FString& DiffAbortThresholdRaw, const FDiffAbortThreshold& DiffAbortThreshold)
{
	if (DiffAbortThreshold.Unit == EDiffAbortThresholdUnits::Absolute)
	{
		if (DiffAbortThreshold.Value < DiffAbortThresholdLimits::MinAbsolute)
		{

			// Clamp DiffAbortThreshold to sane min.
			UE_LOGF(LogBuildPatchTool, Warning, "Requested -DiffAbortThreshold=%ls is below allowed minimum n >= 1GB. Please update your arg to be above limit. Continuing with DiffAbortThreshold=%llu.", *DiffAbortThresholdRaw, DiffAbortThresholdLimits::MinAbsolute);
			return { DiffAbortThresholdLimits::MinAbsolute, EDiffAbortThresholdUnits::Absolute };
		}
		return DiffAbortThreshold;
	}

	check(DiffAbortThreshold.Unit == EDiffAbortThresholdUnits::Percentage);
	if (DiffAbortThreshold.Value < DiffAbortThresholdLimits::MinPercentage)
	{
		UE_LOGF(LogBuildPatchTool, Warning, "Requested -DiffAbortThreshold=%ls is below allowed minimum n >= %llu%%. Please update your arg to be above limit. Continuing with DiffAbortThreshold=%llu%%.", *DiffAbortThresholdRaw, DiffAbortThresholdLimits::MinPercentage, DiffAbortThresholdLimits::MinPercentage);
		return { DiffAbortThresholdLimits::MinPercentage , EDiffAbortThresholdUnits::Percentage };
	}
	return DiffAbortThreshold;
}
