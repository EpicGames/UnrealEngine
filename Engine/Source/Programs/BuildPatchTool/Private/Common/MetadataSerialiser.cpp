// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/MetadataSerialiser.h"

#include "Algo/AnyOf.h"
#include "Algo/MaxElement.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonTypes.h"

namespace
{
	bool IsValid(const FSHA256Signature& Sha256)
	{
		return Algo::AnyOf(Sha256.Signature);
	}
}

// Creating our own LexToString functions for consitency, some in the global scope vary with upper and lowercase on similar types.
namespace ExtractMetaLex
{
	inline FString LexToString(const int64& Num)
	{
		return FString::Printf(TEXT("%lld"), Num);
	}

	inline FString LexToString(const uint64& Num)
	{
		return FString::Printf(TEXT("%llu"), Num);
	}

	inline FString LexToString(const int32& Num)
	{
		return FString::Printf(TEXT("%d"), Num);
	}

	inline FString LexToString(const uint32& Num)
	{
		return FString::Printf(TEXT("%u"), Num);
	}

	inline FString LexToString(const BuildPatchServices::EFeatureLevel& FeatureLevel)
	{
		return FString(::LexToString(FeatureLevel));
	}

	inline FString LexToString(const FMD5Hash& InHash)
	{
		return BytesToHex(InHash.GetBytes(), InHash.GetSize());
	}

	inline FString LexToString(const FSHAHash& InHash)
	{
		return BytesToHex(InHash.Hash, sizeof(InHash.Hash));
	}

	inline FString LexToString(const FSHA256Signature& InHash)
	{
		return BytesToHex(InHash.Signature, sizeof(InHash.Signature));
	}
}

// A struct to help with serialising data in a more divergent way.
struct FFileMetadata
{
	FString Filename;
	TSet<FString> Tags;
	EFileMetaFlags FileMetaFlags = EFileMetaFlags::None;
	FMD5Hash MD5Hash;
	FSHAHash SHA1Hash;
	FSHA256Signature SHA256Hash;
	FString MIMEType;
	int64 FileSize = 0;
};

struct FChunkMetadata
{
	FGuid Guid;
	int64 Size = 0;
	FSHAHash SHA1Hash;
	FGuid EncryptionSecretId;
	TArray<FString> Files;
};

class FHumanMetadataSerialiser
{
public:
	FHumanMetadataSerialiser(FString& InOutputString, BuildPatchServices::EFeatureLevel InFeatureLevel)
		: OutputString(InOutputString)
		, FeatureLevel(InFeatureLevel)
	{
	}

	~FHumanMetadataSerialiser()
	{
		CompleteSection();
	}

	void AddSection(const FString& Section)
	{
		CompleteSection();
		SectionName = Section;
	}

	void AddKeyValue(const FString& Key, const FString& Value)
	{
		SectionKeyValues.Emplace(Key + TEXT(":"), Value);
	}

	void AddKeyValue(const FString& Key, const TArray<FFileMetadata>& Values)
	{
		// Human ignores file list key name, and just lists file detail per line.
		for (const FFileMetadata& FileMetadata : Values)
		{
			FString FileStringValue = FString::Printf(TEXT("\"%s\" size:%lld %s"), *FileMetadata.Filename , FileMetadata.FileSize, *ToAttributeList(FileMetadata.FileMetaFlags, FileMetadata.Tags));
			if (FeatureLevel >= BuildPatchServices::EFeatureLevel::StoresFileMD5HashesAndMIMEType)
			{
				if (FileMetadata.MD5Hash.IsValid())
				{
					FileStringValue += FString::Printf(TEXT("MD5:%s "), *ExtractMetaLex::LexToString(FileMetadata.MD5Hash));
				}
			}
			FileStringValue += FString::Printf(TEXT("SHA1:%s "), *ExtractMetaLex::LexToString(FileMetadata.SHA1Hash));
			if (FeatureLevel >= BuildPatchServices::EFeatureLevel::StoresFileSHA256Hashes)
			{
				if (IsValid(FileMetadata.SHA256Hash))
				{
					FileStringValue += FString::Printf(TEXT("SHA256:%s "), *ExtractMetaLex::LexToString(FileMetadata.SHA256Hash));
				}
			}
			if (FeatureLevel >= BuildPatchServices::EFeatureLevel::StoresFileMD5HashesAndMIMEType)
			{
				FileStringValue += FString::Printf(TEXT("MIME:%s "), FileMetadata.MIMEType.IsEmpty() ? TEXT("None") : *FileMetadata.MIMEType);
			}
			FileStringValue.RemoveFromEnd(TEXT(" "));
			SectionKeyValues.Emplace(TEXT(""), MoveTemp(FileStringValue));
		}
	}

	void AddKeyValue(const FString& Key, const TArray<FChunkMetadata>& Values)
	{
		for (const FChunkMetadata& Metadata : Values)
		{
			FString FileStringValue = FString::Printf(TEXT("DataId:%s SHA1:%s%s size:%lld FilesNum:%d Files:["),
				*Metadata.Guid.ToString(),
				*Metadata.SHA1Hash.ToString(),
				Metadata.EncryptionSecretId.IsValid() ? *(FString(TEXT(" EncryptionSecretId:")) + Metadata.EncryptionSecretId.ToString()) : TEXT(""),
				Metadata.Size,
				Metadata.Files.Num());
			for (const FString& FileName : Metadata.Files)
			{
				FileStringValue += FString::Printf(TEXT("\"%s\" "), *FileName);
			}
			FileStringValue.RemoveFromEnd(TEXT(" "));
			FileStringValue += TEXT("]");
			SectionKeyValues.Emplace(TEXT(""), MoveTemp(FileStringValue));
		}
	}

	void AddKeyValue(const FString& Key, const TSet<FString>& Value, const TCHAR* EmptyRepresentation)
	{
		FString StringifiedValue;
		bool bFirst = true;
		for (const FString& Element : Value)
		{
			if (bFirst)
			{
				bFirst = false;
			}
			else
			{
				StringifiedValue += TEXT(", ");
			}
			if (Element.IsEmpty())
			{
				StringifiedValue += EmptyRepresentation;
			}
			else
			{
				StringifiedValue += Element;
			}
		}
		SectionKeyValues.Emplace(Key + TEXT(":"), StringifiedValue);
	}

	template<typename ValueType>
	void AddKeyValue(const FString& Key, const ValueType& Value)
	{
		AddKeyValue(Key, ExtractMetaLex::LexToString(Value));
	}

private:

	FString ToAttributeList(EFileMetaFlags FileMetaFlags, const TSet<FString>& Tags)
	{
		FString AttributeList;
		if (EnumHasAllFlags(FileMetaFlags, EFileMetaFlags::ReadOnly))
		{
			AttributeList += TEXT("readonly ");
		}
		if (EnumHasAllFlags(FileMetaFlags, EFileMetaFlags::Compressed))
		{
			AttributeList += TEXT("compressed ");
		}
		if (EnumHasAllFlags(FileMetaFlags, EFileMetaFlags::UnixExecutable))
		{
			AttributeList += TEXT("executable ");
		}
		for (const FString& Tag : Tags)
		{
			if (!Tag.IsEmpty())
			{
				AttributeList += TEXT("tag:") + Tag;
				AttributeList += TEXT(" ");
			}
		}
		return AttributeList;
	}

	void CompleteSection()
	{
		if (SectionName.Len() > 0)
		{
			// Add an extra line break if it's not the begining of the output
			if (!OutputString.IsEmpty())
			{
				OutputString += LINE_TERMINATOR;
			}
			OutputString += SectionName;
			OutputString += TEXT(":") LINE_TERMINATOR;
			SectionName.Reset();
		}
		if (SectionKeyValues.Num() > 0)
		{
			// Format and output the file.
			int32 MaxKeyNameLength = Algo::MaxElementBy(SectionKeyValues, [](const TPair<FString, FString>& Element) { return Element.Key.Len(); })->Key.Len();
			if (MaxKeyNameLength > 0)
			{
				MaxKeyNameLength++;
			}
			for (const TPair<FString, FString>& SectionKeyValue : SectionKeyValues)
			{
				OutputString += FString::Printf(TEXT("  %*s%s") LINE_TERMINATOR, -MaxKeyNameLength, *SectionKeyValue.Key, *SectionKeyValue.Value);
			}
			SectionKeyValues.Reset();
		}
	}

private:
	FString& OutputString;
	BuildPatchServices::EFeatureLevel FeatureLevel;
	FString SectionName;
	TArray<TPair<FString, FString>> SectionKeyValues;
};

class FJsonMetadataSerialiser
{
public:
	FJsonMetadataSerialiser(FString& InOutputString, BuildPatchServices::EFeatureLevel InFeatureLevel)
		: OutputString(InOutputString)
		, FeatureLevel(InFeatureLevel)
		, JsonWriter(TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&OutputString))
	{
		JsonWriter->WriteObjectStart();
	}

	~FJsonMetadataSerialiser()
	{
		JsonWriter->WriteObjectEnd();
		JsonWriter->Close();
	}

	void AddKeyValue(const FString& Key, const bool& Value)
	{
		JsonWriter->WriteValue(Key, Value);
	}

	void AddKeyValue(const FString& Key, const uint32& Value)
	{
		const int64 SupportedValue = Value;
		JsonWriter->WriteValue(Key, SupportedValue);
	}

	void AddKeyValue(const FString& Key, const int64& Value)
	{
		JsonWriter->WriteValue(Key, Value);
	}

	void AddKeyValue(const FString& Key, const FString& Value)
	{
		JsonWriter->WriteValue(Key, Value);
	}

	void AddKeyValue(const FString& Key, const BuildPatchServices::EFeatureLevel& Value)
	{
		AddKeyValue(Key, ExtractMetaLex::LexToString(Value));
	}

	void AddKeyValue(const FString& Key, const TArray<FFileMetadata>& Values)
	{
		JsonWriter->WriteArrayStart(Key);
		for (const FFileMetadata& Value : Values)
		{
			JsonWriter->WriteObjectStart();
			AddKeyValue(TEXT("Filename"), Value.Filename);
			AddKeyValue(TEXT("Tags"), Value.Tags, TEXT(""));
			if (FeatureLevel >= BuildPatchServices::EFeatureLevel::StoresFileMD5HashesAndMIMEType)
			{
				if (Value.MD5Hash.IsValid())
				{
					AddKeyValue(TEXT("MD5"), ExtractMetaLex::LexToString(Value.MD5Hash));
				}
			}
			AddKeyValue(TEXT("SHA1"), ExtractMetaLex::LexToString(Value.SHA1Hash));
			if (FeatureLevel >= BuildPatchServices::EFeatureLevel::StoresFileSHA256Hashes)
			{
				if (IsValid(Value.SHA256Hash))
				{
					AddKeyValue(TEXT("SHA256"), ExtractMetaLex::LexToString(Value.SHA256Hash));
				}
			}
			if (FeatureLevel >= BuildPatchServices::EFeatureLevel::StoresFileMD5HashesAndMIMEType)
			{
				AddKeyValue(TEXT("MIME"), Value.MIMEType);
			}
			AddKeyValue(TEXT("FileSize"), Value.FileSize);
			AddKeyValue(TEXT("IsReadOnly"), EnumHasAllFlags(Value.FileMetaFlags, EFileMetaFlags::ReadOnly));
			AddKeyValue(TEXT("IsCompressed"), EnumHasAllFlags(Value.FileMetaFlags, EFileMetaFlags::Compressed));
			AddKeyValue(TEXT("IsExecutable"), EnumHasAllFlags(Value.FileMetaFlags, EFileMetaFlags::UnixExecutable));
			JsonWriter->WriteObjectEnd();
		}
		JsonWriter->WriteArrayEnd();
	}

	void AddKeyValue(const FString& Key, const TArray<FChunkMetadata>& Values)
	{
		JsonWriter->WriteArrayStart(Key);
		for (const FChunkMetadata& Value : Values)
		{
			JsonWriter->WriteObjectStart();
			AddKeyValue(TEXT("Guid"), Value.Guid.ToString());
			AddKeyValue(TEXT("Size"), Value.Size);
			AddKeyValue(TEXT("SHA1"), ExtractMetaLex::LexToString(Value.SHA1Hash));
			if (Value.EncryptionSecretId.IsValid())
			{
				AddKeyValue(TEXT("EncryptionSecretId"), Value.EncryptionSecretId.ToString());
			}
			JsonWriter->WriteArrayStart("Files");
			for (const FString& FileName : Value.Files)
			{
				JsonWriter->WriteValue(*FileName);
			}
			JsonWriter->WriteArrayEnd();
			JsonWriter->WriteObjectEnd();
		}
		JsonWriter->WriteArrayEnd();
	}

	void AddKeyValue(const FString& Key, const TSet<FString>& Values, const TCHAR* EmptyRepresentation)
	{
		JsonWriter->WriteArrayStart(Key);
		for (const FString& Value : Values)
		{
			if (Value.IsEmpty())
			{
				JsonWriter->WriteValue(EmptyRepresentation);
			}
			else
			{
				JsonWriter->WriteValue(Value);
			}
		}
		JsonWriter->WriteArrayEnd();
	}

	// JSON does not use sections
	void AddSection(const FString& Section) { }

private:
	FString& OutputString;
	BuildPatchServices::EFeatureLevel FeatureLevel;
	TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter;
};

template<typename TSerialiser>
void Serialise(const IBuildManifest& Manifest, FString& OutputString)
{
	TSerialiser Serialiser(OutputString, Manifest.GetFeatureLevel());

	TArray<FFileMetadata> FileMetadataList;
	TMap<FString, TSet<FString>> FileToTagsMap;
	TSet<FString> FileTagList;
	Manifest.GetFileTagList(FileTagList);
	for (const FString& Tag : FileTagList)
	{
		for (const FString& TaggedFile : Manifest.GetBuildFileList({ Tag }))
		{
			FileToTagsMap.FindOrAdd(TaggedFile).Add(Tag);
		}
	}
	for (const FString& BuildFile : Manifest.GetBuildFileList())
	{
		FFileMetadata& FileMetadata = FileMetadataList.AddDefaulted_GetRef();
		FileMetadata.Filename = BuildFile;
		FileMetadata.Tags = FileToTagsMap[BuildFile];
		FileMetadata.FileMetaFlags = Manifest.GetFileMetaFlags(BuildFile);
		FileMetadata.SHA1Hash = *Manifest.GetFileSHA1Hash(BuildFile);
		FileMetadata.FileSize = Manifest.GetFileSize(BuildFile);
		if (Manifest.GetFeatureLevel() >= BuildPatchServices::EFeatureLevel::StoresFileMD5HashesAndMIMEType)
		{
			FileMetadata.MD5Hash = *Manifest.GetFileMD5Hash(BuildFile);
			FileMetadata.MIMEType = *Manifest.GetFileMIMEType(BuildFile);
		}
		if (Manifest.GetFeatureLevel() >= BuildPatchServices::EFeatureLevel::StoresFileSHA256Hashes)
		{
			FileMetadata.SHA256Hash = *Manifest.GetFileSHA256Hash(BuildFile);
		}
	}
	Serialiser.AddSection(TEXT("MANIFEST DETAIL"));
	Serialiser.AddKeyValue(TEXT("FeatureLevel"), Manifest.GetFeatureLevel());
	Serialiser.AddKeyValue(TEXT("FeatureLevelInt"), static_cast<int64>(Manifest.GetFeatureLevel()));
	Serialiser.AddKeyValue(TEXT("AppId"), Manifest.GetAppID());
	Serialiser.AddKeyValue(TEXT("AppName"), Manifest.GetAppName());
	Serialiser.AddKeyValue(TEXT("VersionString"), Manifest.GetVersionString());
	Serialiser.AddKeyValue(TEXT("UniqueBuildId"), Manifest.GetUniqueBuildId());
	Serialiser.AddKeyValue(TEXT("LaunchExe"), Manifest.GetLaunchExe());
	Serialiser.AddKeyValue(TEXT("LaunchCommand"), Manifest.GetLaunchCommand());
	if (Manifest.GetFeatureLevel() >= BuildPatchServices::EFeatureLevel::StoresPrerequisiteIds)
	{
		Serialiser.AddKeyValue(TEXT("PrereqIds"), Manifest.GetPrereqIds(), TEXT(""));
	}
	if (Manifest.GetFeatureLevel() >= BuildPatchServices::EFeatureLevel::StoresPrerequisitesInfo)
	{
		Serialiser.AddKeyValue(TEXT("PrereqName"), Manifest.GetPrereqName());
		Serialiser.AddKeyValue(TEXT("PrereqPath"), Manifest.GetPrereqPath());
		Serialiser.AddKeyValue(TEXT("PrereqArgs"), Manifest.GetPrereqArgs());
	}
	if (Manifest.GetFeatureLevel() >= BuildPatchServices::EFeatureLevel::StoresUninstallActions)
	{
		Serialiser.AddKeyValue(TEXT("UninstallActionPath"), Manifest.GetUninstallActionPath());
		Serialiser.AddKeyValue(TEXT("UninstallActionArgs"), Manifest.GetUninstallActionArgs());
	}
	if (Manifest.GetFeatureLevel() >= BuildPatchServices::EFeatureLevel::ChunkEncryptionSupport)
	{
		bool bManifestIsEncrypted;
		FGuid SecretIdUsedByManifest = Manifest.GetEncryptionSecretId(&bManifestIsEncrypted);
		if (SecretIdUsedByManifest.IsValid())
		{
			Serialiser.AddKeyValue(TEXT("EncryptionSecretId"), SecretIdUsedByManifest.ToString());
			Serialiser.AddKeyValue(TEXT("MetadataIsDecrypted"), !bManifestIsEncrypted);
		}
	}
	Serialiser.AddKeyValue(TEXT("DownloadSize"), Manifest.GetDownloadSize());
	Serialiser.AddKeyValue(TEXT("BuildSize"), Manifest.GetBuildSize());
	Serialiser.AddKeyValue(TEXT("FileTagList"), FileTagList, TEXT("(untagged)"));
	for (const FString& CustomFieldName : Manifest.GetCustomFieldNames())
	{
		IManifestFieldPtr CustomField = Manifest.GetCustomField(CustomFieldName);
		if (CustomField.IsValid())
		{
			Serialiser.AddKeyValue(CustomFieldName, CustomField->AsString());
		}
	}
	Serialiser.AddSection(TEXT("MANIFEST FILES"));
	Serialiser.AddKeyValue(TEXT("Files"), FileMetadataList);

	TMap<FGuid, TArray<FString>> ChunksToFiles;
	for (const FString& BuildFile : Manifest.GetBuildFileList())
	{
		TSet<FGuid> RequiredChunks;
		Manifest.GetChunksRequiredForFiles(TSet<FString>{ BuildFile }, RequiredChunks);
		for (const FGuid& ChunkGuid : RequiredChunks)
		{
			ChunksToFiles.FindOrAdd(ChunkGuid).Add(BuildFile);
		}
	}

	TArray<FChunkMetadata> FileChunks;
	FileChunks.Reserve(ChunksToFiles.Num());
	for (TPair<FGuid, TArray<FString>>& Pair : ChunksToFiles)
	{
		FChunkMetadata ChunkInfo;
		ChunkInfo.Guid = Pair.Key;
		Manifest.GetChunkShaHash(ChunkInfo.Guid, ChunkInfo.SHA1Hash);
		Manifest.GetChunkEncryptionSecretId(ChunkInfo.Guid, ChunkInfo.EncryptionSecretId);
		ChunkInfo.Size = Manifest.GetDataSize(ChunkInfo.Guid);
		ChunkInfo.Files.Append(MoveTemp(Pair.Value));
		FileChunks.Push(MoveTemp(ChunkInfo));
	}

	Serialiser.AddSection(TEXT("MANIFEST CHUNKS"));
	Serialiser.AddKeyValue(TEXT("FileChunks"), FileChunks);
}

FString FMetadataSerialiser::SerialiseMetadata(const IBuildManifest& Manifest, EMetadataOutputFormat OutputFormat)
{
	FString OutputString;
	switch (OutputFormat)
	{
	case EMetadataOutputFormat::Human:
		Serialise<FHumanMetadataSerialiser>(Manifest, OutputString);
		break;
	case EMetadataOutputFormat::Json:
		Serialise<FJsonMetadataSerialiser>(Manifest, OutputString);
		break;
	default:
		break;
	}
	return OutputString;
}

bool LexFromString(EMetadataOutputFormat& MetadataOutputFormat, const TCHAR* Buffer)
{
	static_assert(int(EMetadataOutputFormat::InvalidOrMax) == 2, "Please update LexFromString converter once you change enum");

#define RETURN_IF_EQUAL(Value) if (FCString::Stricmp(Buffer, TEXT(#Value)) == 0) { MetadataOutputFormat = EMetadataOutputFormat::Value; return true; }
	const TCHAR* const Prefix = TEXT("EMetadataOutputFormat::");
	const SIZE_T PrefixLen = FCString::Strlen(Prefix);
	if (FCString::Strnicmp(Buffer, Prefix, PrefixLen) == 0)
	{
		Buffer += PrefixLen;
	}
	RETURN_IF_EQUAL(Human);
	RETURN_IF_EQUAL(Json);
	// Did not match
	MetadataOutputFormat = EMetadataOutputFormat::InvalidOrMax;
	return false;
#undef RETURN_IF_EQUAL
}
