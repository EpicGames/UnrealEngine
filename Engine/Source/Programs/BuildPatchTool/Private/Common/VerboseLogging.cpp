// Copyright Epic Games, Inc. All Rights Reserved.
#include "Common/VerboseLogging.h"
#include "BuildPatchTool.h"
#include "Http.h"
#if WITH_ONLINEMCP
#include "OnlineSubsystem.h"
#endif

// BPS Compactify
DECLARE_LOG_CATEGORY_EXTERN(LogDataCompactifier, Log, All);
// BPS Data
DECLARE_LOG_CATEGORY_EXTERN(LogManifestData, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogManifestUObject, Log, All);
// BPS Diffing
DECLARE_LOG_CATEGORY_EXTERN(LogDiffManifests, Log, All);
// BPS Enumeration
DECLARE_LOG_CATEGORY_EXTERN(LogDataEnumeration, Log, All);

static void SetupVerboseLogsFoBPSOthers()
{
	// BPS Compactify
	UE_SET_LOG_VERBOSITY(LogDataCompactifier, All);
	// BPS Data
	UE_SET_LOG_VERBOSITY(LogManifestData, All);
	UE_SET_LOG_VERBOSITY(LogManifestUObject, All);
	// BPS Diffing
	UE_SET_LOG_VERBOSITY(LogDiffManifests, All);
	// BPS Enumeration
	UE_SET_LOG_VERBOSITY(LogDataEnumeration, All);
}

// BPS Generation
DECLARE_LOG_CATEGORY_EXTERN(LogChunkDatabaseWriter, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogChunkDeltaOptimiser, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogChunkWriter, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogCloudEnumeration, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDataScanner, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDeltaEnumeration, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogBuildStreamer, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogFileAttributesParser, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogManifestBuilder, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogManifestBuildStreamer, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogPackageChunkData, Log, All);

static void SetupVerboseLogsFoBPSGeneration()
{
	// BPS Generation
	UE_SET_LOG_VERBOSITY(LogChunkDatabaseWriter, All);
	UE_SET_LOG_VERBOSITY(LogChunkDeltaOptimiser, All);
	UE_SET_LOG_VERBOSITY(LogChunkWriter, All);
	UE_SET_LOG_VERBOSITY(LogCloudEnumeration, All);
	UE_SET_LOG_VERBOSITY(LogDataScanner, All);
	UE_SET_LOG_VERBOSITY(LogDeltaEnumeration, All);
	UE_SET_LOG_VERBOSITY(LogBuildStreamer, All);
	UE_SET_LOG_VERBOSITY(LogFileAttributesParser, All);
	UE_SET_LOG_VERBOSITY(LogManifestBuilder, All);
	UE_SET_LOG_VERBOSITY(LogManifestBuildStreamer, All);
	UE_SET_LOG_VERBOSITY(LogPackageChunkData, All);
}

// BPS Installer
DECLARE_LOG_CATEGORY_EXTERN(LogChunkReferenceTracker, Warning, All);
DECLARE_LOG_CATEGORY_EXTERN(LogCloudChunkSource, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogCustomUninstallAction, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDownloadConnectionCount, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDownloadService, Warning, All);
DECLARE_LOG_CATEGORY_EXTERN(LogInstallChunkSource, Warning, All);
DECLARE_LOG_CATEGORY_EXTERN(LogOptimisedDelta, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogPrerequisites, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogVerifier, Warning, All);

static void SetupVerboseLogsFoBPSInstaller()
{
	// BPS Installer
	UE_SET_LOG_VERBOSITY(LogChunkReferenceTracker, All);
	UE_SET_LOG_VERBOSITY(LogCloudChunkSource, All);
	UE_SET_LOG_VERBOSITY(LogCustomUninstallAction, All);
	UE_SET_LOG_VERBOSITY(LogDownloadConnectionCount, All);
	UE_SET_LOG_VERBOSITY(LogDownloadService, All);
	UE_SET_LOG_VERBOSITY(LogInstallChunkSource, All);
	UE_SET_LOG_VERBOSITY(LogOptimisedDelta, All);
	UE_SET_LOG_VERBOSITY(LogPrerequisites, All);
	UE_SET_LOG_VERBOSITY(LogVerifier, All);
}

// BPS
DECLARE_LOG_CATEGORY_EXTERN(LogBuildPatchFileConstructor, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogPatchGeneration, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogBPSInstallerConfig, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogMergeManifests, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogBuildPatchServices, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogVerifyChunkData, Log, All);

static void SetupVerboseLogsFoBPS()
{
	// BPS
	UE_SET_LOG_VERBOSITY(LogBuildPatchFileConstructor, All);
	UE_SET_LOG_VERBOSITY(LogPatchGeneration, All);
	UE_SET_LOG_VERBOSITY(LogBPSInstallerConfig, All);
	UE_SET_LOG_VERBOSITY(LogMergeManifests, All);
	UE_SET_LOG_VERBOSITY(LogBuildPatchServices, All);
	UE_SET_LOG_VERBOSITY(LogVerifyChunkData, All);
}

#if WITH_ONLINEMCP
// Auth
DECLARE_LOG_CATEGORY_EXTERN(LogAuthManager, Log, All);
// BuildPublishing
DECLARE_LOG_CATEGORY_EXTERN(LogManifestDownload, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogS3ChunkWriter, Log, All);
// Online Modes
DECLARE_LOG_CATEGORY_EXTERN(LogBinaryDeltaOptimise, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogBuildPatchOnlineMcpModule, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogContentProtection, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogCopyBinary, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDeleteBinary, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDiffBinaries, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogGetBinaryMetadata, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogLabelBinary, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogListBinaries, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogReuploadBinary, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogUnlabelBinary, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogUploadBinary, Log, All);
#endif

static void SetupVerboseLogsForOnlineModes()
{
#if WITH_ONLINEMCP
	UE_SET_LOG_VERBOSITY(LogOnline, All);

	// Auth
	UE_SET_LOG_VERBOSITY(LogAuthManager, All);
	// BuildPublishing
	UE_SET_LOG_VERBOSITY(LogManifestDownload, All);
	UE_SET_LOG_VERBOSITY(LogS3ChunkWriter, All);
	// Online Modes
	UE_SET_LOG_VERBOSITY(LogBinaryDeltaOptimise, All);
	UE_SET_LOG_VERBOSITY(LogBuildPatchOnlineMcpModule, All);
	UE_SET_LOG_VERBOSITY(LogContentProtection, All);
	UE_SET_LOG_VERBOSITY(LogCopyBinary, All);
	UE_SET_LOG_VERBOSITY(LogDeleteBinary, All);
	UE_SET_LOG_VERBOSITY(LogDiffBinaries, All);
	UE_SET_LOG_VERBOSITY(LogGetBinaryMetadata, All);
	UE_SET_LOG_VERBOSITY(LogLabelBinary, All);
	UE_SET_LOG_VERBOSITY(LogListBinaries, All);
	UE_SET_LOG_VERBOSITY(LogReuploadBinary, All);
	UE_SET_LOG_VERBOSITY(LogUnlabelBinary, All);
	UE_SET_LOG_VERBOSITY(LogUploadBinary, All);
#endif
}

static inline void SetupVerboseLogsForOnlineTools()
{
	SetupVerboseLogsFoBPSOthers();
	SetupVerboseLogsFoBPSGeneration();
	SetupVerboseLogsFoBPSInstaller();
	SetupVerboseLogsFoBPS();
	SetupVerboseLogsForOnlineModes();
}

void SetupVerboseLogging(bool bVerboseLogsSpecified)
{
	if (bVerboseLogsSpecified)
	{
		UE_SET_LOG_VERBOSITY(LogInit, All);
		UE_SET_LOG_VERBOSITY(LogBuildPatchTool, All);
		UE_SET_LOG_VERBOSITY(LogHttp, All);
		UE_SET_LOG_VERBOSITY(LogMemory, All);
		SetupVerboseLogsForOnlineTools();
	}
	else
	{
		// Increase LogInit verbosity to extra logs
		UE_SET_LOG_VERBOSITY(LogInit, Warning);
	}
}
