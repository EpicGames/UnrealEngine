// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "interfaces/ToolMode.h"

namespace BuildPatchTool
{
	class FMockToolMode
		: public IToolMode
	{
	public:

		virtual ~FMockToolMode() {}

		virtual EReturnCode Execute() override
		{
			return EReturnCode::OK;
		}
	};
}
