// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/CmdLineProcessor.h"
#include "Misc/Paths.h"
#include "Tests/TestHelpers.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FCmdLineProcessorSpec, "BuildPatchTool.Unit", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
// Unit factory to simplify test code
FCmdLineProcessor UnitFactory(TArray<const TCHAR*>);
// Data for each test
FString WorkingDirectory = TEXT("spec/working/dir");
FString ArgToUse;
FString TestData;
// Helpers to tidy up test
TArray<const TCHAR*> CreateTestArgs(const TCHAR*);
FString CreateExpectation(const TCHAR*);
void TestArgsResult(TArray<const TCHAR*>, const FString&);
END_DEFINE_SPEC(FCmdLineProcessorSpec)

void FCmdLineProcessorSpec::Define()
{
	Describe("FCmdLineProcessor", [&]()
	{
		Describe("ProcessDragAndDrop", [&]()
		{
			It("should convert plain file list into full ExtractMetadata args.", [&]()
			{
				TArray<const TCHAR*> DragAndDropArgs = 
				{
					// The first argument is always by the processor since it will always just be the running exe path
					TEXT("some path to exe"),
					TEXT("foo"),
					TEXT("bar"),
				};

				const FString ExpectedCommandLine = FString::Printf(TEXT("-bForceSmokeTests -usehyperthreading -unattended -mode=ExtractMetadata -outputformat=human -openoutput -inputfile=\"%s\" -outputfile=\"%s\" -inputfile=\"%s\" -outputfile=\"%s\""),
					*FPaths::Combine(WorkingDirectory,TEXT("foo")),
					*FPaths::Combine(FPlatformProcess::UserTempDir(), TEXT("foo.txt")),
					*FPaths::Combine(WorkingDirectory, TEXT("bar")),
					*FPaths::Combine(FPlatformProcess::UserTempDir(), TEXT("bar.txt")));

				TestArgsResult(DragAndDropArgs, ExpectedCommandLine);
			});
		});

		Describe("ConvertCommandlinePaths", [&]()
		{
			It("should collapse AppLaunch paths.", [&]()
			{
				ArgToUse = TEXT("AppLaunch");
				
				const FString PutLines127Expected = CreateExpectation(TEXT("subDir2/sub1/putLines_1.2.7.txt"));
				const FString FizzExpected = CreateExpectation(TEXT("fizz.txt"));
				
				TestArgsResult(CreateTestArgs(TEXT("subDir2/../subDir2/sub1/putLines_1.2.7.txt")), PutLines127Expected);
				TestArgsResult(CreateTestArgs(TEXT("subDir2/sub1/../sub1/putLines_1.2.7.txt")), PutLines127Expected);
				TestArgsResult(CreateTestArgs(TEXT("subDir2/sub1/../../subDir2/sub1/putLines_1.2.7.txt")), PutLines127Expected);
				TestArgsResult(CreateTestArgs(TEXT("Relative/Path/2/../../../fizz.txt")), FizzExpected);
				TestArgsResult(CreateTestArgs(TEXT("./fizz.txt")), FizzExpected);
				TestArgsResult(CreateTestArgs(TEXT("fizz.txt")), FizzExpected);
				TestArgsResult(CreateTestArgs(TEXT("subDir2/sub1/putLines_1.2.7.txt")), PutLines127Expected);
				TestArgsResult(CreateTestArgs(TEXT("subDir2\\sub1\\putLines_1.2.7.txt")), PutLines127Expected);
				TestArgsResult(CreateTestArgs(TEXT("Relative\\Path\\2\\..\\..\\..\\fizz.txt")), FizzExpected);
			});

			It("should convert BuildRoot to absolute paths.", [&]()
			{
				ArgToUse = TEXT("BuildRoot");

				const FString AbsoluteSub1 = FPaths::Combine(WorkingDirectory, TEXT("subDir2/sub1/"));
				const FString AbsoluteFizz = FPaths::Combine(WorkingDirectory, TEXT("fizz/"));
				
				const FString Sub1Expected = CreateExpectation(*AbsoluteSub1);
				const FString FizzExpected = CreateExpectation(*AbsoluteFizz);
				const FString MultiPathExpected = CreateExpectation(*FString::Join(TArray<FString>{ AbsoluteSub1, AbsoluteFizz }, TEXT(",")));
				
				TestArgsResult(CreateTestArgs(TEXT("subDir2/../subDir2/sub1/")), Sub1Expected);
				TestArgsResult(CreateTestArgs(TEXT("subDir2/sub1/../sub1/")), Sub1Expected);
				TestArgsResult(CreateTestArgs(TEXT("subDir2/sub1/../../subDir2/sub1/")), Sub1Expected);
				TestArgsResult(CreateTestArgs(TEXT("Relative/Path/2/../../../fizz/")), FizzExpected);
				TestArgsResult(CreateTestArgs(TEXT("./fizz/")), FizzExpected);
				TestArgsResult(CreateTestArgs(TEXT("fizz/")), FizzExpected);
				TestArgsResult(CreateTestArgs(TEXT("subDir2/sub1/")), Sub1Expected);
				TestArgsResult(CreateTestArgs(TEXT("subDir2\\sub1\\")), Sub1Expected);
				TestArgsResult(CreateTestArgs(TEXT("Relative\\Path\\2\\..\\..\\..\\fizz/")), FizzExpected);

				TestArgsResult(CreateTestArgs(TEXT("subDir2/sub1/../sub1/,./fizz/")), MultiPathExpected);
			});

			It("should not affect params provided empty or whitespace.", [&]()
			{
				const TCHAR* EmptyString = TEXT("");
				const TCHAR* Whitespace1 = TEXT(" ");
				const TCHAR* Whitespace2 = TEXT("\t");
				const TCHAR* Whitespace3 = TEXT(" \t   \t");
				
				for (const TCHAR* ArgUnderTest : { TEXT("BuildRoot"), TEXT("AppLaunch") })
				{
					ArgToUse = ArgUnderTest;
					TestArgsResult(CreateTestArgs(EmptyString), CreateExpectation(EmptyString));
					TestArgsResult(CreateTestArgs(Whitespace1), CreateExpectation(Whitespace1));
					TestArgsResult(CreateTestArgs(Whitespace2), CreateExpectation(Whitespace2));
					TestArgsResult(CreateTestArgs(Whitespace3), CreateExpectation(Whitespace3));
				}
			});
		});
	});
}

FCmdLineProcessor FCmdLineProcessorSpec::UnitFactory(TArray<const TCHAR*> Args)
{
	return FCmdLineProcessor(Args.Num(), const_cast<TCHAR**>(Args.GetData()), WorkingDirectory, [](const FString&) {});
}

TArray<const TCHAR*> FCmdLineProcessorSpec::CreateTestArgs(const TCHAR* AppLaunchValue)
{
	TestData = FString::Printf(TEXT("-%s=\"%s\""), *ArgToUse, AppLaunchValue);
	return
	{
		// The first argument is always by the processor since it will always just be the running exe path
		TEXT("some path to exe"),
		TEXT("-mode=UploadBinary"),
		*TestData,
		TEXT("-AppArgs=")
	};
}

FString FCmdLineProcessorSpec::CreateExpectation(const TCHAR* AppLaunchValue)
{
	return FString::Printf(TEXT("-bForceSmokeTests -usehyperthreading -unattended -mode=UploadBinary -%s=\"%s\" -AppArgs="), *ArgToUse, AppLaunchValue);
}

void FCmdLineProcessorSpec::TestArgsResult(TArray<const TCHAR*> Args, const FString& Expected)
{
	FCmdLineProcessor CmdLineProcessor = UnitFactory(Args);

	CmdLineProcessor.ProcessDragAndDrop();
	checkSlow(CmdLineProcessor.ProcessCommandlineFiles());
	checkSlow(CmdLineProcessor.HandleLegacyCommandline());
	checkSlow(CmdLineProcessor.ConvertCommandlinePaths());
	CmdLineProcessor.AddDesiredEngineFlags();

	const FString& EngineString = CmdLineProcessor.GetEngineString();
	checkSlow(TEST_EQUAL(EngineString, Expected));
}

#endif //WITH_DEV_AUTOMATION_TESTS
