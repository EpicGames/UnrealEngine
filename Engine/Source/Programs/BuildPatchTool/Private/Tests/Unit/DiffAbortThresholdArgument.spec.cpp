// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/DiffAbortThresholdArgument.h"

using namespace BuildPatchServices;

#if WITH_DEV_AUTOMATION_TESTS
#if WITH_GOOGLE_TEST

THIRD_PARTY_INCLUDES_START
#include "gtest/gtest.h"
#include "gmock/gmock.h"
THIRD_PARTY_INCLUDES_END

#include "Tests/Mock/ToolMode.mock.h"
#include "Tests/TestHelpers.h"

namespace
{
	struct ArgAndValue
	{
		FString Arg;
		uint64 Value = 0ULL;
	};
}

BEGIN_DEFINE_SPEC(FDiffAbortThresholdArgumentSpec, "BuildPatchTool.Unit", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
	TSharedPtr<testing::NiceMock<BuildPatchTool::FMockToolMode>> MockToolMode;
	
	// The UE automation framework truncated the name before the dot, so the dots are replaced with {dot}.
	static FString FixDotTruncation(const FString& name);
END_DEFINE_SPEC(FDiffAbortThresholdArgumentSpec)

void FDiffAbortThresholdArgumentSpec::Define()
{
	BeforeEach([this]()
	{
		// Create mock dependencies, and the methods used by our service...
		MockToolMode = MakeShared<testing::NiceMock<BuildPatchTool::FMockToolMode>>();
	});
	AfterEach([this]()
	{
		MockToolMode = {};
	});

	Describe("FDiffAbortThresholdArgument", [this]()
	{
		Describe("Parse", [this]()
		{
			for (const FString& Value : TArray<FString>{ TEXT("10.0pc"), TEXT("10pc"), TEXT("10Pct"), TEXT("\"10PCT\""), TEXT("10.4999pc") })
			{
				It(*FString::Format(TEXT("should parse as \"10%\" when -DiffAbortThreshold={0}"), { FixDotTruncation(Value) }), [this, Value]()
				{
					const TArray<FString> Switches = { FString::Format(TEXT("DiffAbortThreshold={0}"), { Value }) };
					const TOptional<FDiffAbortThreshold> DiffAbirtThreshold = BuildPatchTool::FDiffAbortThresholdArgument::Parse(*MockToolMode, Switches);
					ASSERT_TRUE(DiffAbirtThreshold.IsSet());
					EXPECT_EQ(DiffAbirtThreshold->Unit, EDiffAbortThresholdUnits::Percentage);
					EXPECT_EQ(DiffAbirtThreshold->Value, 10ULL);
				});
			}

			for (const FString& Value : TArray<FString>{ TEXT("0.1pc"), TEXT("0.99pc"), TEXT("1pc"), TEXT("\"1.499Pct\"") })
			{
				It(*FString::Format(TEXT("should parse and clamp to 1 percent when -DiffAbortThreshold={0}"), { FixDotTruncation(Value) }), [this, Value]()
				{
					const TArray<FString> Switches = { FString::Format(TEXT("DiffAbortThreshold={0}"), { Value }) };
					const TOptional<FDiffAbortThreshold> DiffAbirtThreshold = BuildPatchTool::FDiffAbortThresholdArgument::Parse(*MockToolMode, Switches);
					ASSERT_TRUE(DiffAbirtThreshold.IsSet());
					EXPECT_EQ(DiffAbirtThreshold->Unit, EDiffAbortThresholdUnits::Percentage);
					EXPECT_EQ(DiffAbirtThreshold->Value, 1ULL);
				});
			}

			for (const FString& Value : TArray<FString>{ TEXT("0pc"), TEXT("0.0001pc"), TEXT("1.4999pc") })
			{
				It(*FString::Format(TEXT("should parse and clamp to 1 percent when -DiffAbortThreshold={0}"), { FixDotTruncation(Value) }), [this, Value]()
				{
					const TArray<FString> Switches = { FString::Format(TEXT("DiffAbortThreshold={0}"), { Value }) };
					const TOptional<FDiffAbortThreshold> DiffAbirtThreshold = BuildPatchTool::FDiffAbortThresholdArgument::Parse(*MockToolMode, Switches);
					ASSERT_TRUE(DiffAbirtThreshold.IsSet());
					EXPECT_EQ(DiffAbirtThreshold->Unit, EDiffAbortThresholdUnits::Percentage);
					EXPECT_EQ(DiffAbirtThreshold->Value, 1ULL);
				});
			}

			for (const FString& Value : TArray<FString>{ TEXT("1000000000"), TEXT("999999999"), TEXT("1"), TEXT("999999kb"), TEXT("976562.49kib"), TEXT("999.999mb"), TEXT("952.721mib"), TEXT("0.999gb"), TEXT("0.9303gib") })
			{
				It(*FString::Format(TEXT("should parse and clamp to 1000000000 bytes when -DiffAbortThreshold={0}"), { FixDotTruncation(Value) }), [this, Value]()
				{
					const TArray<FString> Switches = { FString::Format(TEXT("DiffAbortThreshold={0}"), { Value }) };
					const TOptional<FDiffAbortThreshold> DiffAbirtThreshold = BuildPatchTool::FDiffAbortThresholdArgument::Parse(*MockToolMode, Switches);
					ASSERT_TRUE(DiffAbirtThreshold.IsSet());
					EXPECT_EQ(DiffAbirtThreshold->Unit, EDiffAbortThresholdUnits::Absolute);
					EXPECT_EQ(DiffAbirtThreshold->Value, 1000ULL * 1000ULL * 1000ULL);
				});
			}

			const TArray<ArgAndValue> TestAtguments{ 
				{ TEXT("1000001kb"), 1000001ULL * 1000ULL },
				{ TEXT("1000000kib"), 1000000ULL * 1024ULL },
				{ TEXT("1001mb"), 1001ULL * 1000ULL * 1000ULL },
				{ TEXT("1000mib"), 1000ULL * 1024ULL * 1024ULL },
				{ TEXT("2gb"), 2ULL * 1000ULL * 1000ULL * 1000ULL },
				{ TEXT("1gib"), 1ULL * 1024ULL * 1024ULL * 1024ULL },
			};
			for (const ArgAndValue& Value : TestAtguments)
			{
				It(*FString::Format(TEXT("should parse to {0} bytes when -DiffAbortThreshold={1}"), { Value.Value, FixDotTruncation(Value.Arg) }), [this, Value]()
				{
					const TArray<FString> Switches = { FString::Format(TEXT("DiffAbortThreshold={0}"), { Value.Arg }) };
					const TOptional<FDiffAbortThreshold> DiffAbirtThreshold = BuildPatchTool::FDiffAbortThresholdArgument::Parse(*MockToolMode, Switches);
					ASSERT_TRUE(DiffAbirtThreshold.IsSet());
					EXPECT_EQ(DiffAbirtThreshold->Unit, EDiffAbortThresholdUnits::Absolute);
					EXPECT_EQ(DiffAbirtThreshold->Value, Value.Value);
				});
			}

			for (const FString& Value : TArray<FString>{ TEXT("invalid"), TEXT("999999999invalid"), TEXT("2gibinvalid"), TEXT("999999999.9.9mb"), TEXT(".0.0") })
			{
				It(*FString::Format(TEXT("should parse as unset when -DiffAbortThreshold={0}"), { FixDotTruncation(Value) }), [this, Value]()
				{
					const TArray<FString> Switches = { FString::Format(TEXT("DiffAbortThreshold={0}"), { Value }) };
					const TOptional<FDiffAbortThreshold> DiffAbirtThreshold = BuildPatchTool::FDiffAbortThresholdArgument::Parse(*MockToolMode, Switches);
					ASSERT_FALSE(DiffAbirtThreshold.IsSet());
				});
			}

		});
	});
}

FString FDiffAbortThresholdArgumentSpec::FixDotTruncation(const FString& Value)
{
	return Value.Replace(TEXT("."), TEXT("{dot}"));
}

#endif //WITH_GOOGLE_TEST
#endif //WITH_DEV_AUTOMATION_TESTS
