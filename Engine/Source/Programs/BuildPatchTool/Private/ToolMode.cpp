// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/ToolMode.h"

#include "BuildPatchTool.h"
#include "BuildPatchToolVersion.h"
#include "Common/MetadataSerialiser.h"
#include "Interfaces/IBuildPatchServicesModule.h"
#include "Misc/CommandLine.h"
#include "ToolModes/ToolModesHelp.h"

namespace BuildPatchTool
{
	class FHelpToolMode : public IToolMode
	{
	public:
		FHelpToolMode(const TCHAR* InCommandLine)
			: CommandLine(InCommandLine)
		{}

		virtual ~FHelpToolMode()
		{}

		virtual EReturnCode Execute() override
		{
			bool bRequestedHelp = FParse::Param(CommandLine, TEXT("help"));

			// Output generic help info
			if (!bRequestedHelp)
			{
				UE_LOGF(LogBuildPatchTool, Error, "ERROR: No mode was detected on the commandline, please check the provided args. You can use -help for information on available modes.");
				UE_LOGF(LogBuildPatchTool, Display, "");
			}
			PrintHelp<FBptBaseToolModeHelp>();

			// Error if this wasn't just a help request
			return bRequestedHelp ? EReturnCode::OK : EReturnCode::UnknownToolMode;
		}

	private:
		const TCHAR* CommandLine;
	};

	IToolMode* FToolModeFactory::Create(IBuildPatchServicesModule& BpsInterface, const TCHAR* CommandLine)
	{
		TMap<FString, FToolModeCreateFunc> Modes;
		GetModeRegistryDelegate().Broadcast(Modes);

		// Create the correct tool mode for the commandline given.
		FString ToolModeValue;
		if (FParse::Value(CommandLine, TEXT("mode="), ToolModeValue))
		{
			if (const FToolModeCreateFunc* ToolModeCreator = Modes.Find(*ToolModeValue))
			{
				return (*ToolModeCreator)(BpsInterface, CommandLine);
			}
		}

		// If we didn't find a toolmode, check for empty mode name for overriden help.
		if (const FToolModeCreateFunc* ToolModeCreator = Modes.Find(TEXT("")))
		{
			return (*ToolModeCreator)(BpsInterface, CommandLine);
		}

		// No previous attempts found a tool mode. Create the generic help, which will return ok if -help was provided else return UnknownToolMode error
		return new FHelpToolMode(CommandLine);
	}

	FToolModeFactory::FModeRegistryDelegate& FToolModeFactory::GetModeRegistryDelegate()
	{
		static FToolModeFactory::FModeRegistryDelegate ModeRegistryDelegate;
		return ModeRegistryDelegate;
	}
}
