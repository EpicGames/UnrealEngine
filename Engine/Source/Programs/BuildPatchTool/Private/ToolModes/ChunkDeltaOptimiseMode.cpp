// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolModes/ChunkDeltaOptimiseMode.h"

#include "BuildPatchTool.h"
#include "Common/DiffAbortThresholdArgument.h"
#include "Interfaces/ToolMode.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Optional.h"
#include "Misc/Paths.h"
#include "ToolModes/ToolModesHelp.h"

using namespace BuildPatchTool;

class FChunkDeltaOptimiseMode : public IToolMode
{
public:
	FChunkDeltaOptimiseMode(IBuildPatchServicesModule& InBpsInterface, const TCHAR* InCommandLine)
		: BpsInterface(InBpsInterface)
		, CommandLine(InCommandLine)
	{}

	virtual ~FChunkDeltaOptimiseMode()
	{}

	virtual EReturnCode Execute() override
	{
		using namespace BuildPatchServices;

		// Parse command line.
		if (ProcessCommandline() == false)
		{
			return EReturnCode::ArgumentProcessingError;
		}

		// Print help if requested.
		if (bHelp)
		{
			PrintHelp<FChunkDeltaOptimiseModeHelp>();
			return EReturnCode::OK;
		}

		LogConfig();

		// Register secrets.
		for (const TPair<FGuid, TArray<uint8>>& IdKeyPair : EncryptionSecrets)
		{
			BpsInterface.RegisterEncryptionSecret(IdKeyPair.Key, IdKeyPair.Value);
		}

		// Set any commandline based config.
		GConfig->SetInt(TEXT("Portal.BuildPatch"), TEXT("ChunkRetries"), MaxRetryCount, GEngineIni);

		// Setup and run.
		FChunkDeltaOptimiserConfiguration Configuration;
		Configuration.ManifestAUri = ManifestA;
		Configuration.ManifestBUri = ManifestB;
		Configuration.OutputCloudDirectory = CloudDir;
		Configuration.DownloadCloudDirectories = DownloadCloudDirectories;
		Configuration.ScanWindowSize = ScanWindowSize;
		Configuration.OutputChunkSize = OutputChunkSize;
		Configuration.DiffAbortThreshold = DiffAbortThreshold;
		Configuration.DeltaFilenameTrailer = DeltaFilenameTrailer;
		Configuration.SourceDetailsLogFilename = SourceDetailsLogFilename;
		Configuration.ManifestBTagAliases = MoveTemp(ManifestBTagAliases);
		Configuration.ManifestATagAliases = MoveTemp(ManifestATagAliases);
		Configuration.ManifestAIgnoreTags = MoveTemp(ManifestAIgnoreTags);
		Configuration.ManifestBIgnoreTags = MoveTemp(ManifestBIgnoreTags);

		// Run the build generation.
		IChunkDeltaOptimiserRef ChunkDeltaOptimiser = BpsInterface.CreateChunkDeltaOptimiser(Configuration);

		bool bSuccess = ChunkDeltaOptimiser->Run();

		return bSuccess ? EReturnCode::OK : EReturnCode::ToolFailure;
	}

private:
	// returns false on an error, not if the switch is present! Check OutMap for presence.
	bool ParseManyToOneMap(const TCHAR* SwitchName, const TArray<FString>& Switches, TMap<FString, FString>& OutMap)
	{
		TArray<FString> RawStrings;
		FString SwitchNameWithEq = FString(SwitchName) + TEXT("=");
		if (ParseSwitches(*SwitchNameWithEq, RawStrings, Switches))
		{
			for (const FString& ManyToOneString : RawStrings)
			{
				FString From, To;
				if (!ManyToOneString.Split(TEXT(";"), &From, &To))
				{
					UE_LOGF(LogBuildPatchTool, Error, "%ls must contain a ';' separating the 'from' tags and the 'to' tag (%ls)", SwitchName, *ManyToOneString);
					return false;
				}

				TArray<FString> FromArray;
				From.ParseIntoArray(FromArray, TEXT(","));
				if (FromArray.Num() == 0)
				{
					UE_LOGF(LogBuildPatchTool, Error, "%ls must contain a comma separated list of 'from' tags (%ls)", SwitchName, *ManyToOneString);
					return false;
				}
				To.TrimStartAndEndInline();
				if (To.Len() == 0)
				{
					UE_LOGF(LogBuildPatchTool, Error, "%ls must contain a 'to' tag after a ';' (%ls)", SwitchName, *ManyToOneString);
					return false;
				}

				for (const FString& FromEntry : FromArray)
				{
					FString Trimmed = FromEntry.TrimStartAndEnd();
					if (Trimmed.Len() == 0)
					{
						UE_LOGF(LogBuildPatchTool, Error, "%ls contained an empty entry in the 'from' list (%ls)", SwitchName, *ManyToOneString);
						return false;
					}
					OutMap.Add(MoveTemp(Trimmed), To);
				}
			}
		}
		return true;
	}

	bool ParseCommaSetSwitch(const TCHAR* SwitchName, const TArray<FString>& Switches, TSet<FString>& OutValues)
	{
		TArray<FString> RawStrings;
		FString SwitchNameWithEq = FString(SwitchName) + TEXT("=");
		if (ParseSwitches(*SwitchNameWithEq, RawStrings, Switches))
		{
			for (const FString& ListString : RawStrings)
			{
				TArray<FString> Entries;
				ListString.ParseIntoArray(Entries, TEXT(","));
				if (Entries.Num() == 0)
				{
					UE_LOGF(LogBuildPatchTool, Error, "%ls must contain a comma separated list tags (%ls)", SwitchName, *ListString);
					return false;
				}

				for (const FString& Entry : Entries)
				{
					FString Trimmed = Entry.TrimStartAndEnd();
					if (Trimmed.Len() == 0)
					{
						UE_LOGF(LogBuildPatchTool, Error, "%ls contained an empty entry (%ls)", SwitchName, *ListString);
						return false;
					}

					OutValues.Emplace(MoveTemp(Trimmed));
				}
			}
		}
		return true;
	}

	bool ProcessCommandline()
	{
#define PARSE_SWITCH(Switch) ParseSwitch(TEXT(#Switch "="), Switch, Switches)
#define _PARSE_REQUIRED_SWITCH(Switch, bExpect) \
		if (PARSE_SWITCH(Switch) != bExpect) \
		{ \
			UE_LOG(LogBuildPatchTool, Error, #Switch SENTENCE_END); \
			bParametersOk = false; \
		}
#define PARSE_REQUIRED_SWITCH(Switch) _PARSE_REQUIRED_SWITCH(Switch, true)
#define PARSE_UNEXPECTED_SWITCH(Switch) _PARSE_REQUIRED_SWITCH(Switch, false)

		TArray<FString> Tokens, Switches;
		FCommandLine::Parse(CommandLine, Tokens, Switches);

		bHelp = ParseOption(TEXT("help"), Switches);
		if (bHelp)
		{
			return true;
		}

		// TODOBPTONLINE
#undef SENTENCE_END
#define SENTENCE_END TEXT(" is a required parameter for offline mode.")
		bool bParametersOk = true;
		PARSE_REQUIRED_SWITCH(ManifestA);
		PARSE_REQUIRED_SWITCH(ManifestB);
#undef SENTENCE_END
#define SENTENCE_END TEXT(" should only be used when uploading to Epic services.")
		FString BuildVersionA, BuildVersionB;
		PARSE_UNEXPECTED_SWITCH(BuildVersionA);
		PARSE_UNEXPECTED_SWITCH(BuildVersionB);
#undef SENTENCE_END
#define SENTENCE_END TEXT(" is only supported when Epic services are available. Download an official build from Epic to enable this functionality.")
		FString ClientId, OrganizationId, ProductId;
		PARSE_UNEXPECTED_SWITCH(ClientId);
		PARSE_UNEXPECTED_SWITCH(OrganizationId);
		PARSE_UNEXPECTED_SWITCH(ProductId);

		if (!bParametersOk)
		{
			return false;
		}

		if (ManifestA.IsEmpty() || ManifestB.IsEmpty())
		{
			UE_LOGF(LogBuildPatchTool, Error, "ManifestA/B arguments can't be empty");
			return false;
		}
		
		NormalizeUriFile(ManifestA);
		NormalizeUriFile(ManifestB);

		// Get optional values.
		const bool bHasCloudDir = PARSE_SWITCH(CloudDir);
		const bool bHasDownloadCloudDirectory = ParseSwitches(TEXT("DownloadCloudDirectory="), DownloadCloudDirectories, Switches);

		PARSE_SWITCH(SourceDetailsLogFilename);
		PARSE_SWITCH(DeltaFilenameTrailer);
		PARSE_SWITCH(ScanWindowSize);
		PARSE_SWITCH(OutputChunkSize);
		PARSE_SWITCH(MaxRetryCount);
		if (!bHasCloudDir)
		{
			CloudDir = FPaths::GetPath(ManifestB);
		}
		if (!bHasDownloadCloudDirectory)
		{
			DownloadCloudDirectories.Add(FPaths::GetPath(ManifestB));
		}

		NormalizeUriPath(CloudDir);
		for (FString& DownloadCloudDirectory : DownloadCloudDirectories)
		{
			NormalizeUriPath(DownloadCloudDirectory);
		}

		// Clamp ScanWindowSize to sane range.
		const uint32 RequestedScanWindowSize = ScanWindowSize;
		ScanWindowSize = FMath::Clamp<uint32>(ScanWindowSize, 128, 128*1024);
		if (RequestedScanWindowSize != ScanWindowSize)
		{
			UE_LOGF(LogBuildPatchTool, Warning, "Requested -ScanWindowSize=%u is outside of allowed range 128KiB >= n >= 128b. Please update your arg to be within range. Continuing with %u.", RequestedScanWindowSize, ScanWindowSize);
		}

		// Clamp OutputChunkSize to sane range.
		const uint32 RequestedOutputChunkSize = OutputChunkSize;
		OutputChunkSize = FMath::Clamp<uint32>(OutputChunkSize, 1000000, 10*1024*1024);
		if (RequestedOutputChunkSize != OutputChunkSize)
		{
			UE_LOGF(LogBuildPatchTool, Warning, "Requested -OutputChunkSize=%u is outside of allowed range 10MiB >= n >= 1MB. Please update your arg to be within range. Continuing with %u.", RequestedOutputChunkSize, OutputChunkSize);
		}
		DiffAbortThreshold = FDiffAbortThresholdArgument::Parse(*this, Switches);

		// Parse all the secret key and ID pairs.
		FString ParseErrorMessage;
		if (!ParsePairs(TEXT("EncryptionSecretId="), TEXT("EncryptionSecretKey="), Switches, EncryptionSecrets, ParseErrorMessage) && !ParseErrorMessage.IsEmpty())
		{
			UE_LOGF(LogBuildPatchTool, Error, "%ls", *ParseErrorMessage);
			return false;
		}

		// Parse any tag aliases we want to use. This is for constructing multiple optimized deltas
		// where we know we have a larger installation set to draw from instead of just the same tag.
		// Format is: -ManifestATagAliases="tag1,tag2,tag3,tag4;alias"
		if (!ParseManyToOneMap(TEXT("ManifestATagAliases"), Switches, ManifestATagAliases))
		{
			return false; // already logged.
		}
		if (!ParseManyToOneMap(TEXT("ManifestBTagAliases"), Switches, ManifestBTagAliases))
		{
			return false; // already logged.
		}

		// Parse tags that should not affect whether a file is allowed to produce matches or not. For example,
		// without this:
		// filename_1.txt tag:one tag:two
		// filename_2.txt tag:one
		// The files will not be able to provide matches during patching to each other as their tagsets do not match.
		// However if you pass "two" as an ignore tag, it will be removed from consideration and the matcher will only
		// se tag:one for filename_1.txt. You can specify different ignore tags for the two manifests used for optimization.
		if (!ParseCommaSetSwitch(TEXT("ManifestAIgnoreTags"), Switches, ManifestAIgnoreTags))
		{
			return false; // already logged.
		}
		if (!ParseCommaSetSwitch(TEXT("ManifestBIgnoreTags"), Switches, ManifestBIgnoreTags))
		{
			return false; // already logged.
		}

		return true;
#undef SENTENCE_END
#undef PARSE_UNEXPECTED_SWITCH
#undef PARSE_REQUIRED_SWITCH
#undef _PARSE_REQUIRED_SWITCH
#undef PARSE_SWITCH
	}

	void LogConfig() const
	{
		UE_LOGF(LogBuildPatchTool, Log, "-----Configuration for ChunkDeltaOptimise------");
		UE_LOGF(LogBuildPatchTool, Log, "   DiffAbortThreshold: %ls", *LexToString(DiffAbortThreshold));
		UE_LOGF(LogBuildPatchTool, Log, "   ManifestA: %ls", *ManifestA);
		UE_LOGF(LogBuildPatchTool, Log, "   ManifestB: %ls", *ManifestB);
		UE_LOGF(LogBuildPatchTool, Log, "   CloudDir: %ls", *CloudDir);
		for (const FString& DownloadCloudDirectory : DownloadCloudDirectories)
		{
			UE_LOGF(LogBuildPatchTool, Log, "   DownloadCloudDirectory: %ls", *DownloadCloudDirectory);
		}
		UE_LOGF(LogBuildPatchTool, Log, "   ScanWindowSize: %u", ScanWindowSize);
		UE_LOGF(LogBuildPatchTool, Log, "   OutputChunkSize: %u", OutputChunkSize);
		UE_LOGF(LogBuildPatchTool, Log, "   MaxRetryCount: %u", MaxRetryCount);
		for (const TPair<FGuid, TArray<uint8>>& EncryptionSecret : EncryptionSecrets)
		{
			UE_LOGF(LogBuildPatchTool, Log, "   AvailableSecretId: %ls", *EncryptionSecret.Key.ToString());
		}
		if (ManifestATagAliases.Num())
		{
			// For logging we reverse the map
			TMap<FString, TArray<FString>> BackwardsMap;
			for (const TPair<FString, FString>& Pair : ManifestATagAliases)
			{
				BackwardsMap.FindOrAdd(Pair.Value).Add(Pair.Key);
			}
			for (const TPair<FString, TArray<FString>>& TagAlias : BackwardsMap)
			{
				UE_LOGF(LogBuildPatchTool, Log, "   ManifestATagAliases: %ls -> %ls", *FString::Join(TagAlias.Value, TEXT(",")), *TagAlias.Key);
			}
		}
		if (ManifestBTagAliases.Num())
		{
			// For logging we reverse the map
			TMap<FString, TArray<FString>> BackwardsMap;
			for (const TPair<FString, FString>& Pair : ManifestBTagAliases)
			{
				BackwardsMap.FindOrAdd(Pair.Value).Add(Pair.Key);
			}
			for (const TPair<FString, TArray<FString>>& TagAlias : BackwardsMap)
			{
				UE_LOGF(LogBuildPatchTool, Log, "   ManifestBTagAliases: %ls -> %ls", *FString::Join(TagAlias.Value, TEXT(",")), *TagAlias.Key);
			}
		}
		if (DeltaFilenameTrailer.Len())
		{
			UE_LOGF(LogBuildPatchTool, Log, "   DeltaFilenameTrailer: %ls", *DeltaFilenameTrailer);
		}
		if (SourceDetailsLogFilename.Len())
		{
			UE_LOGF(LogBuildPatchTool, Log, "   SourceDetailsLogFilename: %ls", *SourceDetailsLogFilename);
		}
		if (ManifestAIgnoreTags.Num())
		{
			UE_LOGF(LogBuildPatchTool, Log, "   ManifestAIgnoreTags: %ls", *FString::Join(ManifestAIgnoreTags, TEXT(",")));
		}
		if (ManifestBIgnoreTags.Num())
		{
			UE_LOGF(LogBuildPatchTool, Log, "   ManifestBIgnoreTags: %ls", *FString::Join(ManifestBIgnoreTags, TEXT(",")));
		}
	}

private:
	static TCHAR const* const MODE_NAME;
	IBuildPatchServicesModule& BpsInterface;
	const TCHAR* CommandLine;
	bool bHelp = false;
	FString ManifestA;
	FString ManifestB;
	FString CloudDir;
	TArray<FString> DownloadCloudDirectories;
	uint32 ScanWindowSize = 8191U;
	uint32 OutputChunkSize = 1024U * 1024U;
	uint32 MaxRetryCount = 30;
	TOptional<BuildPatchServices::FDiffAbortThreshold> DiffAbortThreshold;
	TMap<FString, FString> ManifestBTagAliases;
	TMap<FString, FString> ManifestATagAliases;
	TSet<FString> ManifestAIgnoreTags;
	TSet<FString> ManifestBIgnoreTags;
	FString DeltaFilenameTrailer;
	FString SourceDetailsLogFilename;
	TMap<FGuid, TArray<uint8>> EncryptionSecrets;
};

IMPLEMENT_BPT_MODE(ChunkDeltaOptimise, FChunkDeltaOptimiseMode);
IMPLEMENT_BPT_MODE_ALIAS(ChunkDeltaOptimize, FChunkDeltaOptimiseMode);