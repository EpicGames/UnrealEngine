// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolModes/ChunkDirectoryMode.h"

#include "Algo/Transform.h"
#include "BuildPatchFeatureLevel.h"
#include "BuildPatchTool.h"
#include "Common/MetadataSerialiser.h"
#include "Interfaces/IBuildPatchServicesModule.h"
#include "Interfaces/IDirectoryChunker.h"
#include "Interfaces/ToolMode.h"
#include "Misc/Base64.h"
#include "Misc/CommandLine.h"
#include "Misc/DefaultValueHelper.h"
#include "Misc/FileHelper.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/Paths.h"
#include "Serialization/MemoryWriter.h"
#include "ToolModes/ToolModesHelp.h"

class FChunkDirectoryToolMode : public BuildPatchTool::IToolMode
{
public:
	FChunkDirectoryToolMode(IBuildPatchServicesModule& BpsInterface, const TCHAR* InCommandLine);
	virtual BuildPatchTool::EReturnCode Execute() override;
private:
	bool ProcessCommandline();
	void LogConfig() const;
	void StoreMetadata(const IBuildManifest& Manifest) const;
private:
	FString GenerateRandomManifestFilename();
	bool ParsePrereqIds(const FString& ParamValue, TSet<FString>& OutPrereqIds);
	bool ParseCustomField(const FString& Switch, TMap<FString, FVariant>& Fields);

private:
	static TCHAR const* const MODE_NAME;
	IBuildPatchServicesModule& BpsInterface;
	const TCHAR* CommandLine;
	bool bHelp;
	FString FeatureLevel;
	FString BuildRoot;
	FString CloudDir;
	FString AppId = TEXT("0");
	FString ArtifactId;
	FString BuildVersion;
	FString AppLaunch;
	FString AppArgs;
	FString PrereqIds;
	TSet<FString> PrereqIdsSet;
	FString PrereqName;
	FString PrereqPath;
	FString PrereqArgs;
	FString UninstallActionPath;
	FString UninstallActionArgs;
	FString FileList;
	FString FileIgnoreList;
	FString FileAttributeList;
	FString DataAgeThreshold;
	uint32 ChunkWindowSize = 1048576;
	bool bIgnoreOtherWindowSizes = false;
	bool bFollowSymlinks = false;
	bool bResaveKnownChunks = false;
	bool bSHA1Only = false;
	TArray<uint8> EncryptionSecretKey;
	FGuid EncryptionSecretId;
	TMap<FString, FVariant> CustomFields;
	FString OutputFilename;
	FString MetadataOutput;
	FString FileSortOrder;
	FString ManifestSHA1Hash;
	BuildPatchServices::IDirectoryChunkerPtr DirectoryChunker;
};

IMPLEMENT_BPT_MODE(ChunkBuildDirectory, FChunkDirectoryToolMode);

FChunkDirectoryToolMode::FChunkDirectoryToolMode(IBuildPatchServicesModule& InBpsInterface, const TCHAR* InCommandLine)
	: BpsInterface(InBpsInterface)
	, CommandLine(InCommandLine)
{}

BuildPatchTool::EReturnCode FChunkDirectoryToolMode::Execute()
{
	// Parse commandline
	if (ProcessCommandline() == false)
	{
		return BuildPatchTool::EReturnCode::ArgumentProcessingError;
	}

	// Print help if requested
	if (bHelp)
	{
		PrintHelp<BuildPatchTool::FChunkDirectoryToolModeHelp>();
		return BuildPatchTool::EReturnCode::OK;
	}

	LogConfig();

	// Check build root exists
	if (!FPaths::DirectoryExists(BuildRoot))
	{
		UE_LOGF(LogBuildPatchTool, Error, "Specified build root directory does not exist.");
		return BuildPatchTool::EReturnCode::DirectoryNotFound;
	}

	// Check existence of the file list
	if (!FileList.IsEmpty() && !FPaths::FileExists(FileList))
	{
		UE_LOGF(LogBuildPatchTool, Error, "Provided file list was not found %ls", *FileList);
		return BuildPatchTool::EReturnCode::FileNotFound;
	}

	// Check existence of file ignore list
	if (!FileIgnoreList.IsEmpty() && !FPaths::FileExists(FileIgnoreList))
	{
		UE_LOGF(LogBuildPatchTool, Error, "Provided file ignore list was not found %ls", *FileIgnoreList);
		return BuildPatchTool::EReturnCode::FileNotFound;
	}

	// Check existence of file attributes list
	if (!FileAttributeList.IsEmpty() && !FPaths::FileExists(FileAttributeList))
	{
		UE_LOGF(LogBuildPatchTool, Error, "Provided file attribute list was not found %ls", *FileAttributeList);
		return BuildPatchTool::EReturnCode::FileNotFound;
	}

	// Default the OutputFilename if not provided
	if (OutputFilename.IsEmpty())
	{
		OutputFilename = FDefaultValueHelper::RemoveWhitespaces(ArtifactId + BuildVersion) + TEXT(".manifest");
	}
	// Otherwise check the parameter
	else
	{
		if (OutputFilename.Contains(TEXT("/")))
		{
			UE_LOGF(LogBuildPatchTool, Error, "Provided OutputFilename should be clean filename only. Invalid arg: %ls", *OutputFilename);
			return BuildPatchTool::EReturnCode::ArgumentProcessingError;
		}
	}

	// Setup and run
	BuildPatchServices::FDirectoryChunkerConfiguration Configuration;
	LexFromString(Configuration.FeatureLevel, *FeatureLevel);
	Configuration.RootDirectory = BuildRoot;
	Configuration.AppId = TCString<TCHAR>::Atoi64(*AppId);
	Configuration.AppName = ArtifactId;
	Configuration.BuildVersion = BuildVersion;
	Configuration.LaunchExe = AppLaunch;
	Configuration.LaunchCommand = AppArgs;
	Configuration.InputListFile = FileList;
	Configuration.IgnoreListFile = FileIgnoreList;
	Configuration.AttributeListFile = FileAttributeList;
	Configuration.PrereqIds = PrereqIdsSet;
	Configuration.PrereqName = PrereqName;
	Configuration.PrereqPath = PrereqPath;
	Configuration.PrereqArgs = PrereqArgs;
	Configuration.UninstallActionPath = UninstallActionPath;
	Configuration.UninstallActionArgs = UninstallActionArgs;
	Configuration.DataAgeThreshold = TCString<TCHAR>::Atod(*DataAgeThreshold);
	Configuration.bShouldHonorReuseThreshold = DataAgeThreshold.IsEmpty() == false;
	Configuration.OutputChunkWindowSize = ChunkWindowSize;
	Configuration.bShouldMatchAnyWindowSize = !bIgnoreOtherWindowSizes && Configuration.FeatureLevel >= BuildPatchServices::EFeatureLevel::VariableSizeChunks;
	Configuration.bResaveKnownChunks = bResaveKnownChunks;
	Configuration.CustomFields = CustomFields;
	Configuration.CloudDirectory = CloudDir;
	Configuration.OutputFilename = OutputFilename;
	Configuration.bFollowSymlinks = bFollowSymlinks;
	Configuration.bSHA1Only = bSHA1Only;
	Configuration.EncryptionSecretKey = EncryptionSecretKey;
	Configuration.EncryptionSecretId = EncryptionSecretId;

	const uint32 DefaultChunkWindowSize = Configuration.OutputChunkWindowSize;

	// Set the desired file sort order
	if (!FileSortOrder.IsEmpty())
	{
		Configuration.FileSortOrder = BuildPatchServices::EFileSortOrder::InvalidOrMax;
		LexFromString(Configuration.FileSortOrder, *FileSortOrder);
		if (Configuration.FileSortOrder == BuildPatchServices::EFileSortOrder::InvalidOrMax)
		{
			UE_LOGF(LogBuildPatchTool, Error, "Provided FileSortOrder is not recognised. Invalid arg: -FileSortOrder=%ls", *FileSortOrder);
			return BuildPatchTool::EReturnCode::ArgumentProcessingError;
		}
	}

	// Check feature compatibility
	if (Configuration.FeatureLevel == BuildPatchServices::EFeatureLevel::Invalid)
	{
		UE_LOGF(LogBuildPatchTool, Error, "Provided FeatureLevel is not recognised. Invalid arg: -FeatureLevel=%ls", *FeatureLevel);
		return BuildPatchTool::EReturnCode::ArgumentProcessingError;
	}
	const bool bHasCustomFields = Configuration.CustomFields.Num() > 0;
	if (Configuration.FeatureLevel < BuildPatchServices::EFeatureLevel::CustomFields && bHasCustomFields)
	{
		UE_LOGF(LogBuildPatchTool, Error, "Invalid args: FeatureLevel %ls is not compatible with Custom, CustomInt, or CustomFloat.", LexToString(Configuration.FeatureLevel));
		return BuildPatchTool::EReturnCode::ArgumentProcessingError;
	}
	const bool bHasAnyPrereqInfo = !PrereqName.IsEmpty() || !PrereqPath.IsEmpty() || !PrereqArgs.IsEmpty();
	if (Configuration.FeatureLevel < BuildPatchServices::EFeatureLevel::StoresPrerequisitesInfo && bHasAnyPrereqInfo)
	{
		UE_LOGF(LogBuildPatchTool, Error, "Invalid args: FeatureLevel %ls is not compatible with PrereqName, PrereqPath, or PrereqArgs.", LexToString(Configuration.FeatureLevel));
		return BuildPatchTool::EReturnCode::ArgumentProcessingError;
	}
	const bool bHasPrereqIds = Configuration.PrereqIds.Num() > 0;
	if (Configuration.FeatureLevel < BuildPatchServices::EFeatureLevel::StoresPrerequisiteIds && bHasPrereqIds)
	{
		UE_LOGF(LogBuildPatchTool, Error, "Invalid args: FeatureLevel %ls is not compatible with PrereqIds.", LexToString(Configuration.FeatureLevel));
		return BuildPatchTool::EReturnCode::ArgumentProcessingError;
	}
	const bool bHasUninstallAction = !Configuration.UninstallActionPath.IsEmpty() || !Configuration.UninstallActionArgs.IsEmpty();
	if (Configuration.FeatureLevel < BuildPatchServices::EFeatureLevel::StoresUninstallActions && bHasUninstallAction)
	{
		UE_LOGF(LogBuildPatchTool, Error, "Invalid args: FeatureLevel %ls is not compatible with either UninstallActionPath or UninstallActionArgs.", LexToString(Configuration.FeatureLevel));
		return BuildPatchTool::EReturnCode::ArgumentProcessingError;
	}
	const bool bHasNonDefaultWindowSize = Configuration.OutputChunkWindowSize != DefaultChunkWindowSize;
	if (Configuration.FeatureLevel < BuildPatchServices::EFeatureLevel::VariableSizeChunks && bHasNonDefaultWindowSize)
	{
		UE_LOGF(LogBuildPatchTool, Error, "Invalid args: FeatureLevel %ls is not compatible with -ChunkWindowSize=%u.", LexToString(Configuration.FeatureLevel), Configuration.OutputChunkWindowSize);
		return BuildPatchTool::EReturnCode::ArgumentProcessingError;
	}
	const bool bUsingEncryption = Configuration.EncryptionSecretKey.Num() > 0 || Configuration.EncryptionSecretId.IsValid();
	if (Configuration.FeatureLevel < BuildPatchServices::EFeatureLevel::ManifestEncryptionSupport && bUsingEncryption)
	{
		UE_LOGF(LogBuildPatchTool, Error, "Invalid args: FeatureLevel %ls is not compatible with encryption arguments.", LexToString(Configuration.FeatureLevel));
		return BuildPatchTool::EReturnCode::ArgumentProcessingError;
	}


	bool bSuccess;

	// Run the build generation
	UE_LOGF(LogBuildPatchTool, Display, "Beginning chunk generation of version %ls of artifact %ls. Build root: %ls", *Configuration.BuildVersion, *Configuration.AppName, *Configuration.RootDirectory);
	DirectoryChunker = BpsInterface.CreateDirectoryChunker(Configuration);

	DirectoryChunker->OnManifestGeneratedDecrypted().AddRaw(this, &FChunkDirectoryToolMode::StoreMetadata);

	bSuccess = DirectoryChunker->Run();

	if (!bSuccess)
	{
		UE_LOGF(LogBuildPatchTool, Error, "An unexpected error occurred generating chunks.");
		return BuildPatchTool::EReturnCode::ToolFailure;
	}

	UE_LOGF(LogBuildPatchTool, Display, "Chunk generation complete.");

	return BuildPatchTool::EReturnCode::OK;
}

bool FChunkDirectoryToolMode::ParsePrereqIds(const FString& PreReqIdString, TSet<FString>& OutPrereqIds)
{
	if (PreReqIdString.Contains(Constants::Slash)
		|| PreReqIdString.Contains(Constants::Backslash)
		|| PreReqIdString.Contains(Constants::DoubleQuote))
	{
		return false;
	}

	TArray<FString> ParamValues;
	PreReqIdString.ParseIntoArray(ParamValues, *Constants::Comma);
	for (FString& ParamValue : ParamValues)
	{
		ParamValue.TrimStartAndEndInline();
	}
	OutPrereqIds.Append(ParamValues);
	return true;
}

bool FChunkDirectoryToolMode::ParseCustomField(const FString& Switch, TMap<FString, FVariant>& Fields)
{
	FString Type, Left, Right;

	Switch.Split(Constants::Equals, &Type, &Right);
	Type.ToLowerInline();
	Right.Split(Constants::Equals, &Left, &Right);
	Left.TrimStartAndEndInline();
	if (Type.Equals(Constants::Custom, ESearchCase::CaseSensitive))
	{
		// No need to trim string fields
		CustomFields.Add(Left, FVariant(Right));
	}
	else if (Type.Equals(Constants::CustomInt, ESearchCase::CaseSensitive))
	{
		Right.TrimStartAndEndInline();
		if (!Right.IsNumeric())
		{
			UE_LOGF(LogBuildPatchTool, Error, "An error occurred processing numeric token from commandline -%ls", *Switch);
			return false;
		}
		CustomFields.Add(Left, FVariant(TCString<TCHAR>::Atoi64(*Right)));
	}
	else if (Type.Equals(Constants::CustomFloat, ESearchCase::CaseSensitive))
	{
		Right.TrimStartAndEndInline();
		if (!Right.IsNumeric())
		{
			UE_LOGF(LogBuildPatchTool, Error, "An error occurred processing numeric token from commandline -%ls", *Switch);
			return false;
		}
		CustomFields.Add(Left, FVariant(TCString<TCHAR>::Atod(*Right)));
	}

	return true;
}

bool FChunkDirectoryToolMode::ProcessCommandline()
{
#define PARSE_SWITCH(Switch) ParseSwitch(TEXT(#Switch "="), Switch, Switches)
#define PARSE_UNEXPECTED_SWITCH(Switch) \
		if (PARSE_SWITCH(Switch) != false) \
		{ \
			UE_LOG(LogBuildPatchTool, Error, #Switch SENTENCE_END); \
			bParametersOk = false; \
		}

	TArray<FString> Tokens, Switches;
	FCommandLine::Parse(CommandLine, Tokens, Switches);

	bHelp = ParseOption(TEXT("help"), Switches);
	if (bHelp)
	{
		return true;
	}

	// Get all required parameters
	if (!(PARSE_SWITCH(CloudDir)
		&& PARSE_SWITCH(BuildRoot)
		&& PARSE_SWITCH(ArtifactId)
		&& PARSE_SWITCH(BuildVersion)
		&& PARSE_SWITCH(AppLaunch)
		&& PARSE_SWITCH(AppArgs)))
	{
		UE_LOGF(LogBuildPatchTool, Error, "CloudDir, BuildRoot, ArtifactId, BuildVersion, AppLaunch, and AppArgs are required parameters");
		return false;
	}

	// Grab the FeatureLevel. This is required param - no defaults.
	const bool bFeatureLevelPresent = PARSE_SWITCH(FeatureLevel);
	FeatureLevel.TrimStartAndEndInline();

	const BuildPatchServices::EFeatureLevel OnlineFeatureLevelNumeric = BuildPatchServices::EFeatureLevel::FirstOptimisedDelta;
	BuildPatchServices::EFeatureLevel FeatureLevelToSetNumeric = BuildPatchServices::EFeatureLevel::Invalid;

	if (!bFeatureLevelPresent)
	{
		UE_LOGF(LogBuildPatchTool, Error, "FeatureLevel was not provided. Please provide the FeatureLevel commandline argument which matches the existing client support.");
		return false;
	}
	else
	{
		LexFromString(FeatureLevelToSetNumeric, *FeatureLevel);
		if (BuildPatchServices::EFeatureLevel::Invalid == FeatureLevelToSetNumeric)
		{
			UE_LOGF(LogBuildPatchTool, Error, "Invalid arg: -FeatureLevel=%ls. FeatureLevel was provided, but not recognized. Please provide the FeatureLevel commandline argument which matches the existing client support.", *FeatureLevel);
			return false;
		}
	}

	FeatureLevel = LexToString(FeatureLevelToSetNumeric);

	NormalizeUriPath(CloudDir);
	NormalizeUriPath(BuildRoot);
	NormalizeUriFile(AppLaunch);

	// Trim directory names, because low level APIs will report trailing whitespace as fine for existence checks, but fail on actually writing data.
	CloudDir.TrimStartAndEndInline();
	BuildRoot.TrimStartAndEndInline();

	if (BuildRoot.IsEmpty())
	{
		UE_LOGF(LogBuildPatchTool, Error, "No build root was provided.");
		return false;
	}

	if (CloudDir.IsEmpty())
	{
		UE_LOGF(LogBuildPatchTool, Error, "No cloud directory was provided.");
		return false;
	}

	// Ensure CloudDir and BuildRoot paths are both absolute paths and normalized before comparing them to one another.
	CloudDir = FPaths::ConvertRelativePathToFull(CloudDir);
	BuildRoot = FPaths::ConvertRelativePathToFull(BuildRoot);

	// Check that the CloudDir and BuildRoot directories are not identical.
	if (CloudDir.Equals(BuildRoot))
	{
		UE_LOGF(LogBuildPatchTool, Error, "The BuildRoot and CloudDir directories must be different.");
		return false;
	}

	// Check that the CloudDir is not a child of the BuildRoot.
	if (CloudDir.StartsWith(BuildRoot) && (CloudDir[BuildRoot.Len()] == TEXT('/')))
	{
		UE_LOGF(LogBuildPatchTool, Error, "The CloudDir directory must not be a child of the BuildRoot directory.");
		return false;
	}

	// Check that the BuildRoot is not a child of the CloudDir.
	if (BuildRoot.StartsWith(CloudDir) && (BuildRoot[CloudDir.Len()] == TEXT('/')))
	{
		UE_LOGF(LogBuildPatchTool, Error, "The BuildRoot directory must not be a child of the CloudDir directory.");
		return false;
	}

	// Get optional parameters
	bool bParametersOk = true;
	FString ClientId, OrganizationId, ProductId;
	// TODOBPTONLINE
#define SENTENCE_END TEXT(" is only supported when Epic services are available. Download an official build from Epic to enable this functionality.")
	PARSE_UNEXPECTED_SWITCH(ClientId);
	PARSE_UNEXPECTED_SWITCH(OrganizationId);
	PARSE_UNEXPECTED_SWITCH(ProductId);
	if (!bParametersOk)
	{
		return false;
	}
#undef SENTENCE_END

	PARSE_SWITCH(AppId);
	PARSE_SWITCH(FileList);
	PARSE_SWITCH(FileIgnoreList);
	PARSE_SWITCH(FileAttributeList);
	PARSE_SWITCH(PrereqIds);
	PARSE_SWITCH(PrereqName);
	PARSE_SWITCH(PrereqPath);
	PARSE_SWITCH(PrereqArgs);
	PARSE_SWITCH(UninstallActionPath);
	PARSE_SWITCH(UninstallActionArgs);
	PARSE_SWITCH(DataAgeThreshold);
	PARSE_SWITCH(ChunkWindowSize);
	bIgnoreOtherWindowSizes = ParseOption(TEXT("IgnoreOtherWindowSizes"), Switches);
	PARSE_SWITCH(OutputFilename);
	PARSE_SWITCH(MetadataOutput);
	PARSE_SWITCH(FileSortOrder);
	bFollowSymlinks = ParseOption(TEXT("FollowSymlinks"), Switches);
	bResaveKnownChunks = ParseOption(TEXT("ResaveKnownChunks"), Switches);
	bSHA1Only = ParseOption(TEXT("SHA1Only"), Switches);

	// Parse encryption inputs
	PARSE_SWITCH(EncryptionSecretKey);
	FString EncryptionSecretIdString;
	ParseSwitch(TEXT("EncryptionSecretId="), EncryptionSecretIdString, Switches);
	ParseValue(EncryptionSecretIdString, EncryptionSecretId);

	NormalizeUriFile(FileList);
	NormalizeUriFile(FileIgnoreList);
	NormalizeUriFile(FileAttributeList);
	NormalizeUriFile(PrereqPath);
	NormalizeUriFile(UninstallActionPath);
	NormalizeUriFile(OutputFilename);
	NormalizeUriFile(MetadataOutput);

	// Check manifest file extension.
	const TCHAR* ManifestExtension = TEXT(".manifest");
	if (!OutputFilename.IsEmpty() && !OutputFilename.EndsWith(ManifestExtension))
	{
		OutputFilename += ManifestExtension;
	}

	// Clamp ChunkWindowSize to sane range.
	const uint32 RequestedChunkWindowSize = ChunkWindowSize;
	ChunkWindowSize = FMath::Clamp<uint32>(ChunkWindowSize, 32000, 10485760);
	if (RequestedChunkWindowSize != ChunkWindowSize)
	{
		UE_LOGF(LogBuildPatchTool, Warning, "Requested -ChunkWindowSize=%u is outside of allowed range 10485760 >= n >= 32000. Please update your args to be within range. Continuing with %u.", RequestedChunkWindowSize, ChunkWindowSize);
	}

	// Check numeric values
	if (!AppId.IsEmpty() && (!AppId.IsNumeric() || TCString<TCHAR>::Atoi64(*AppId) < 0))
	{
		UE_LOGF(LogBuildPatchTool, Error, "An error occurred processing numeric token from commandline -AppId=%ls", *AppId);
		return false;
	}
	if (!DataAgeThreshold.IsEmpty() && (!DataAgeThreshold.IsNumeric() || TCString<TCHAR>::Atod(*DataAgeThreshold) < 0.0))
	{
		UE_LOGF(LogBuildPatchTool, Error, "An error occurred processing numeric token from commandline -DataAgeThreshold=%ls", *DataAgeThreshold);
		return false;
	}

	// Check the correct number of bytes was received for a secret key
	switch (EncryptionSecretKey.Num())
	{
		case 0:
		case 32:
		break;

		default:
			UE_LOGF(LogBuildPatchTool, Error, "For encryption, EncryptionSecretKey must be 256 bit.");
			return false;
	}

	// Check that if an encryption ID was provided, that it was successfully parsed
	if (EncryptionSecretIdString.Len() > 0 && !EncryptionSecretId.IsValid())
	{
		UE_LOGF(LogBuildPatchTool, Error, "For encryption, EncryptionSecretId must be a valid 32 character hexidecimal GUID.");
		return false;
	}

	// Check neither or both encryption values are received
	if (EncryptionSecretKey.Num() > 0 != EncryptionSecretId.IsValid())
	{
		UE_LOGF(LogBuildPatchTool, Error, "For encryption, both EncryptionSecretKey and EncryptionSecretId must be provided.");
		return false;
	}

	// Get custom fields to add to manifest
	// These are optional, but a failure to parse one is an error
	for (const FString& Switch : Switches)
	{
		if (Switch.StartsWith(Constants::Custom) && !ParseCustomField(Switch, CustomFields))
		{
			return false;
		}
	}

	if (!PrereqIds.IsEmpty() && !ParsePrereqIds(PrereqIds, PrereqIdsSet))
	{
		UE_LOGF(LogBuildPatchTool, Error, "An error occurred processing comma-separated list from commandline -PrereqIds=%ls", *PrereqIds);
		return false;
	}

	return true;
#undef PARSE_SWITCH
#undef PARSE_UNEXPECTED_SWITCH
}

void FChunkDirectoryToolMode::LogConfig() const
{
	UE_LOGF(LogBuildPatchTool, Log, "-----Configuration for ChunkDirectory------");
	UE_LOGF(LogBuildPatchTool, Log, "   ArtifactId: %ls", *ArtifactId);

	UE_LOGF(LogBuildPatchTool, Log, "   FeatureLevel: %ls", *FeatureLevel);
	UE_LOGF(LogBuildPatchTool, Log, "   BuildRoot: %ls", *BuildRoot);
	UE_LOGF(LogBuildPatchTool, Log, "   CloudDir: %ls", *CloudDir);
	UE_LOGF(LogBuildPatchTool, Log, "   AppId: %ls", *AppId);
	UE_LOGF(LogBuildPatchTool, Log, "   BuildVersion: %ls", *BuildVersion);
	UE_LOGF(LogBuildPatchTool, Log, "   AppLaunch: %ls", *AppLaunch);
	UE_LOGF(LogBuildPatchTool, Log, "   AppArgs: %ls", *AppArgs);

	FString PrereqIdString = FString::Join(PrereqIdsSet.Array(), TEXT(","));
	UE_LOGF(LogBuildPatchTool, Log, "   PrereqIds: %ls", *PrereqIdString);
	UE_LOGF(LogBuildPatchTool, Log, "   PrereqName: %ls", *PrereqName);
	UE_LOGF(LogBuildPatchTool, Log, "   PrereqPath: %ls", *PrereqPath);
	UE_LOGF(LogBuildPatchTool, Log, "   PrereqArgs: %ls", *PrereqArgs);
	UE_LOGF(LogBuildPatchTool, Log, "   UninstallActionPath: %ls", *UninstallActionPath);
	UE_LOGF(LogBuildPatchTool, Log, "   UninstallActionArgs: %ls", *UninstallActionArgs);
	UE_LOGF(LogBuildPatchTool, Log, "   FileList: %ls", *FileList);
	UE_LOGF(LogBuildPatchTool, Log, "   FileIgnoreList: %ls", *FileIgnoreList);
	UE_LOGF(LogBuildPatchTool, Log, "   FileAttributeList: %ls", *FileAttributeList);
	UE_LOGF(LogBuildPatchTool, Log, "   DataAgeThreshold: %ls", *DataAgeThreshold);
	UE_LOGF(LogBuildPatchTool, Log, "   ChunkWindowSize: %u", ChunkWindowSize);
	UE_LOGF(LogBuildPatchTool, Log, "   MetadataOutput: %ls", *MetadataOutput);
	UE_LOGF(LogBuildPatchTool, Log, "   bIgnoreOtherWindowSizes: %ls", bIgnoreOtherWindowSizes ? TEXT("true") : TEXT("false"));
	UE_LOGF(LogBuildPatchTool, Log, "   bFollowSymlinks: %ls", bFollowSymlinks ? TEXT("true") : TEXT("false"));
	UE_LOGF(LogBuildPatchTool, Log, "   bResaveKnownChunks: %ls", bResaveKnownChunks ? TEXT("true") : TEXT("false"));
	UE_LOGF(LogBuildPatchTool, Log, "   bSHA1Only: %ls", bSHA1Only ? TEXT("true") : TEXT("false"));
	if (EncryptionSecretId.IsValid())
	{
		UE_LOGF(LogBuildPatchTool, Log, "   EncryptionSecretId: %ls", *LexToString(EncryptionSecretId));
	}

	for (const TPair<FString, FVariant>& CustomField : CustomFields)
	{
		switch (CustomField.Value.GetType())
		{
		case EVariantTypes::String:
			UE_LOGF(LogBuildPatchTool, Log, "   Custom Field %ls: %ls", *CustomField.Get<0>(), *CustomField.Get<1>().GetValue<FString>());
			break;
		case EVariantTypes::Int64:
			UE_LOGF(LogBuildPatchTool, Log, "   Custom Field %ls: %lld", *CustomField.Get<0>(), CustomField.Get<1>().GetValue<int64>());
			break;
		case EVariantTypes::Double:
			UE_LOGF(LogBuildPatchTool, Log, "   Custom Field %ls: %f", *CustomField.Get<0>(), CustomField.Get<1>().GetValue<double>());
			break;

		default:
			break;
		}
	}
}

FString FChunkDirectoryToolMode::GenerateRandomManifestFilename()
{
	FString Result;
	TArray<uint8> OutputFilenameBytes;
	FMemoryWriter MemoryWriter(OutputFilenameBytes);

	// First add some random bytes for security
	for (uint8 i = 0; i < 6; ++i)
	{
		uint8 RandomByte = FMath::Rand();
		MemoryWriter << RandomByte;
	}

	// Now add a GUID for uniqueness
	FGuid OutputFilenameGuid = FGuid::NewGuid();
	MemoryWriter << OutputFilenameGuid;

	MemoryWriter.Close();

	// Serialize to base 64, and replace unsafe filename chars with safer alternatives.
	Result = FBase64::Encode(OutputFilenameBytes);
	Result.ReplaceInline(TEXT("+"), TEXT("-"));
	Result.ReplaceInline(TEXT("/"), TEXT("_"));
	while (Result.RemoveFromEnd(TEXT("=")));
	Result.Append(TEXT(".manifest"));
	return Result;
}

void FChunkDirectoryToolMode::StoreMetadata(const IBuildManifest& Manifest) const
{
	if (!MetadataOutput.IsEmpty())
	{
		EMetadataOutputFormat OutputFormat = MetadataOutput.EndsWith(TEXT(".txt")) ? EMetadataOutputFormat::Human : EMetadataOutputFormat::Json;
		FString OutputString = FMetadataSerialiser::SerialiseMetadata(Manifest, OutputFormat);
		// Save the output
		FFileHelper::SaveStringToFile(OutputString, *MetadataOutput);
	}
}
