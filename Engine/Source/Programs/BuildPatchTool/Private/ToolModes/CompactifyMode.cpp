// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolModes/CompactifyMode.h"

#include "BuildPatchSettings.h"
#include "BuildPatchTool.h"
#include "Interfaces/IBuildPatchServicesModule.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "ToolModes/ToolModesHelp.h"

using namespace BuildPatchTool;

class FCompactifyToolMode : public IToolMode
{
public:
	FCompactifyToolMode(IBuildPatchServicesModule& InBpsInterface, const TCHAR* InCommandLine)
		: BpsInterface(InBpsInterface)
		, CommandLine(InCommandLine)
	{}

	virtual ~FCompactifyToolMode()
	{}

	virtual EReturnCode Execute() override
	{
		// Parse commandline
		if (ProcessCommandline() == false)
		{
			return EReturnCode::ArgumentProcessingError;
		}

		// Print help if requested
		if (bHelp)
		{
			PrintHelp<FCompactifyToolModeHelp>();
			return EReturnCode::OK;
		}

		LogConfig();

		// Grab options
		BuildPatchServices::FCompactifyConfiguration Configuration;
		Configuration.CloudDirectory = CloudDir;
		Configuration.DataAgeThreshold = TCString<TCHAR>::Atod(*DataAgeThreshold);
		Configuration.DeletedChunkLogFile = DeletedChunkLogFile;
		Configuration.bRunPreview = bPreview;

		// Run the compactify routine
		bool bSuccess = BpsInterface.CompactifyCloudDirectory(Configuration);
		return bSuccess ? EReturnCode::OK : EReturnCode::ToolFailure;
	}

private:

	bool ProcessCommandline()
	{
#define PARSE_SWITCH(Switch) ParseSwitch(TEXT(#Switch "="), Switch, Switches)
		TArray<FString> Tokens, Switches;
		FCommandLine::Parse(CommandLine, Tokens, Switches);

		bHelp = ParseOption(TEXT("help"), Switches);
		if (bHelp)
		{
			return true;
		}

		// Get all required parameters
		if (!(PARSE_SWITCH(CloudDir)
		   && PARSE_SWITCH(DataAgeThreshold)))
		{
			UE_LOGF(LogBuildPatchTool, Error, "CloudDir and DataAgeThreshold are required parameters");
			return false;
		}

		if (CloudDir.IsEmpty())
		{
			UE_LOGF(LogBuildPatchTool, Error, "CloudDir argument can't be empty");
			return false;
		}

		NormalizeUriPath(CloudDir);

		// Check required numeric values
		if (!DataAgeThreshold.IsNumeric() || TCString<TCHAR>::Atod(*DataAgeThreshold) < 0.0)
		{
			UE_LOGF(LogBuildPatchTool, Error, "An error occurred processing numeric token from commandline -DataAgeThreshold=%ls", *DataAgeThreshold);
			return false;
		}

		// Get optional parameters
		PARSE_SWITCH(DeletedChunkLogFile);
		NormalizeUriFile(DeletedChunkLogFile);
		bPreview = ParseOption(TEXT("preview"), Switches);

		return true;
#undef PARSE_SWITCH
	}

	void LogConfig() const
	{
		UE_LOGF(LogBuildPatchTool, Log, "-----Configuration for CompactifyMode------");
		UE_LOGF(LogBuildPatchTool, Log, "   CloudDir: %ls", *CloudDir);
		UE_LOGF(LogBuildPatchTool, Log, "   DataAgeThreshold: %ls", *DataAgeThreshold);
		UE_LOGF(LogBuildPatchTool, Log, "   DeletedChunkLogFile: %ls", *DeletedChunkLogFile);
		UE_LOGF(LogBuildPatchTool, Log, "   bPreview: %ls", bPreview ? TEXT("true") : TEXT("false"));
	}

private:
	static TCHAR const* const MODE_NAME;
	IBuildPatchServicesModule& BpsInterface;
	const TCHAR* CommandLine;
	bool bHelp;
	FString CloudDir;
	FString DataAgeThreshold;
	FString DeletedChunkLogFile;
	bool bPreview;
};

IMPLEMENT_BPT_MODE(Compactify, FCompactifyToolMode);
