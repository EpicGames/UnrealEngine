// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolModes/DiffManifestMode.h"

#include "BuildPatchTool.h"
#include "Interfaces/IBuildPatchServicesModule.h"
#include "Interfaces/ToolMode.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "ToolModes/ToolModesHelp.h"

using namespace BuildPatchTool;

class FDiffManifestToolMode : public IToolMode
{
public:
	FDiffManifestToolMode(IBuildPatchServicesModule& InBpsInterface, const TCHAR* InCommandLine)
		: BpsInterface(InBpsInterface)
		, CommandLine(InCommandLine)
	{}

	virtual ~FDiffManifestToolMode()
	{}

	virtual EReturnCode Execute() override
	{
		// Parse commandline.
		if (ProcessCommandLine() == false)
		{
			return EReturnCode::ArgumentProcessingError;
		}

		// Print help if requested.
		if (bHelp)
		{
			PrintHelp<FDiffManifestToolModeHelp>();
			return EReturnCode::OK;
		}

		LogConfig();

		// Register secrets.
		for (const TPair<FGuid, TArray<uint8>>& IdKeyPair : EncryptionSecrets)
		{
			BpsInterface.RegisterEncryptionSecret(IdKeyPair.Key, IdKeyPair.Value);
		}

		// Setup and run.
		BuildPatchServices::FDiffManifestsConfiguration Configuration;
		Configuration.ManifestAUri = ManifestA;
		Configuration.ManifestBUri = ManifestB;
		Configuration.TagSetA = InstallTagsA;
		Configuration.TagSetB = InstallTagsB;
		Configuration.CompareTagSetsA = CompareTagSets;
		Configuration.CompareTagSetsB = CompareNewTagSets;
		Configuration.OutputFilePath = OutputFile;
		Configuration.OutputPatchDescriptorPath = PatchDescriptorPath;
		Configuration.bOnlyPatchDescriptors = bOnlyPatchDescriptors;
		Configuration.bRequireOptimizedDelta = bRequireOptimizedDelta;
		Configuration.DeltaFilenameTrailer = DeltaFilenameTrailer;
		Configuration.bSimulateInstallTimes = bSimulateInstallTimes;

		// Run the merge manifest routine.
		bool bSuccess = BpsInterface.DiffManifests(Configuration);
		return bSuccess ? EReturnCode::OK : EReturnCode::ToolFailure;
	}

private:

	bool ProcessCommandLine()
	{
#define PARSE_SWITCH(Switch) ParseSwitch(TEXT(#Switch "="), Switch, Switches)
#define _PARSE_REQUIRED_SWITCH(Switch, bExpect) \
		if (PARSE_SWITCH(Switch) != bExpect) \
		{ \
			UE_LOG(LogBuildPatchTool, Error, #Switch SENTENCE_END); \
			bParametersOk = false; \
		}
#define PARSE_REQUIRED_SWITCH(Switch) _PARSE_REQUIRED_SWITCH(Switch, true)
#define PARSE_UNEXPECTED_SWITCH(Switch) _PARSE_REQUIRED_SWITCH(Switch, false)

		TArray<FString> Tokens, Switches;
		FCommandLine::Parse(CommandLine, Tokens, Switches);

		bHelp = ParseOption(TEXT("help"), Switches);
		if (bHelp)
		{
			return true;
		}

		// local variables to check for unexpected parameters
		FString BuildVersionA, BuildVersionB, ClientId, OrganizationId, ProductId;
#define SENTENCE_END TEXT(" is a required parameter for offline mode.")
		bool bParametersOk = true;
		PARSE_REQUIRED_SWITCH(ManifestA);
		PARSE_REQUIRED_SWITCH(ManifestB);
#undef SENTENCE_END
#define SENTENCE_END TEXT(" should only be used when uploading to Epic services.")
		PARSE_UNEXPECTED_SWITCH(BuildVersionA);
		PARSE_UNEXPECTED_SWITCH(BuildVersionB);
#undef SENTENCE_END
#define SENTENCE_END TEXT(" is only supported when Epic services are available. Download an official build from Epic to enable this functionality.")
		PARSE_UNEXPECTED_SWITCH(ClientId);
		PARSE_UNEXPECTED_SWITCH(OrganizationId);
		PARSE_UNEXPECTED_SWITCH(ProductId);
#undef SENTENCE_END

		if (!bParametersOk)
		{
			return false;
		}

		if (ManifestA.IsEmpty() || ManifestB.IsEmpty())
		{
			UE_LOGF(LogBuildPatchTool, Error, "ManifestA/B arguments can't be empty");
			return false;
		}
		
		NormalizeUriFile(ManifestA);
		NormalizeUriFile(ManifestB);

		// Get optional parameters
		PARSE_SWITCH(InstallTagsA);
		PARSE_SWITCH(InstallTagsB);
		PARSE_SWITCH(OutputFile);
		NormalizeUriFile(OutputFile);

		PARSE_SWITCH(PatchDescriptorPath);
		NormalizeUriFile(PatchDescriptorPath);

		PARSE_SWITCH(DeltaFilenameTrailer);
		
		bRequireOptimizedDelta = ParseOption(TEXT("RequireOptimizedDelta"), Switches);
		bOnlyPatchDescriptors = ParseOption(TEXT("OnlyPatchDescriptors"), Switches);
		bSimulateInstallTimes = ParseOption(TEXT("SimulateInstallTimes"), Switches);
		bool bSkipInstallTime = ParseOption(TEXT("SkipInstallTime"), Switches);
		if (bSkipInstallTime)
		{
			bSimulateInstallTimes = false;
		}

		TArray<FString> CompareTagsArray;
		ParseSwitches(TEXT("CompareTagSet="), CompareTagsArray, Switches);

		for (const FString& CompareTagsList : CompareTagsArray)
		{
			CompareTagSets.Add(ProcessTagList(CompareTagsList));
		}

		TArray<FString> CompareNewTagsArray;
		ParseSwitches(TEXT("CompareNewTagSet="), CompareNewTagsArray, Switches);

		for (const FString& CompareTagsList : CompareNewTagsArray)
		{
			CompareNewTagSets.Add(ProcessTagList(CompareTagsList));
		}

		// Parse all the secret key and ID pairs.
		FString ParseErrorMessage;
		if (!ParsePairs(TEXT("EncryptionSecretId="), TEXT("EncryptionSecretKey="), Switches, EncryptionSecrets, ParseErrorMessage) && !ParseErrorMessage.IsEmpty())
		{
			UE_LOGF(LogBuildPatchTool, Error, "%ls", *ParseErrorMessage);
			return false;
		}

		return true;
#undef PARSE_UNEXPECTED_SWITCH
#undef PARSE_REQUIRED_SWITCH
#undef _PARSE_REQUIRED_SWITCH
#undef PARSE_SWITCH
	}

	TSet<FString> ProcessTagList(const FString& TagCommandLine) const
	{
		TArray<FString> TagArray;
		TagCommandLine.ParseIntoArray(TagArray, TEXT(","), false);
		for (FString& Tag : TagArray)
		{
			Tag.TrimStartAndEndInline();
		}
		if (TagArray.Num() == 0)
		{
			TagArray.Add(TEXT(""));
		}
		return TSet<FString>(MoveTemp(TagArray));
	}

	void LogConfig() const
	{
		UE_LOGF(LogBuildPatchTool, Log, "-----Configuration for DiffManifests------");
		UE_LOGF(LogBuildPatchTool, Log, "   ManifestA: %ls", *ManifestA);
		UE_LOGF(LogBuildPatchTool, Log, "   ManifestB: %ls", *ManifestB);
		TArray<FString> CompareTagSetStrings;
		for (const TSet<FString>& CompareTagSet : CompareTagSets)
		{
			CompareTagSetStrings.Add(FString::Join(CompareTagSet, TEXT(",")));
		}
		TArray<FString> CompareNewTagSetStrings;
		for (const TSet<FString>& CompareTagSet : CompareNewTagSets)
		{
			CompareNewTagSetStrings.Add(FString::Join(CompareTagSet, TEXT(",")));
		}
		UE_LOGF(LogBuildPatchTool, Log, "   CompareTagSets: %ls", *FString::Join(CompareTagSetStrings, TEXT(";")));
		UE_LOGF(LogBuildPatchTool, Log, "   CompareNewTagSets: %ls", *FString::Join(CompareNewTagSetStrings, TEXT(";")));
		UE_LOGF(LogBuildPatchTool, Log, "   InstallTagsA: %ls", *FString::Join(InstallTagsA.Array(), TEXT(",")));
		UE_LOGF(LogBuildPatchTool, Log, "   InstallTagsB: %ls", *FString::Join(InstallTagsB.Array(), TEXT(",")));
		UE_LOGF(LogBuildPatchTool, Log, "   OutputFile: %ls", *OutputFile);
		UE_LOGF(LogBuildPatchTool, Log, "   OutputPatchDescriptorPath: %ls", *PatchDescriptorPath);
		UE_LOGF(LogBuildPatchTool, Log, "   OnlyPatchDescriptors: %ls", *LexToString(bOnlyPatchDescriptors));
		UE_LOGF(LogBuildPatchTool, Log, "   RequireOptimizedDelta: %ls", *LexToString(bRequireOptimizedDelta));
		UE_LOGF(LogBuildPatchTool, Log, "   SimulateInstallTimes: %ls", *LexToString(bSimulateInstallTimes));
		if (DeltaFilenameTrailer.Len())
		{
			UE_LOGF(LogBuildPatchTool, Log, "   DeltaFilenameTrailer: %ls", *DeltaFilenameTrailer);		
		}
		for (const TPair<FGuid, TArray<uint8>>& EncryptionSecret : EncryptionSecrets)
		{
			UE_LOGF(LogBuildPatchTool, Log, "   AvailableSecretId: %ls", *EncryptionSecret.Key.ToString());
		}
	}

private:
	static TCHAR const* const MODE_NAME;
	IBuildPatchServicesModule& BpsInterface;
	const TCHAR* CommandLine;
	bool bHelp;
	FString ManifestA;
	FString ManifestB;
	TArray<TSet<FString>> CompareTagSets;
	TArray<TSet<FString>> CompareNewTagSets;
	TSet<FString> InstallTagsA;
	TSet<FString> InstallTagsB;
	FString OutputFile;
	FString PatchDescriptorPath;
	bool bRequireOptimizedDelta;
	bool bOnlyPatchDescriptors;
	bool bSimulateInstallTimes;
	FString DeltaFilenameTrailer;
	TMap<FGuid, TArray<uint8>> EncryptionSecrets;
};

IMPLEMENT_BPT_MODE(DiffManifests, FDiffManifestToolMode);
