// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolModes/EnumerationMode.h"

#include "BuildPatchTool.h"
#include "Interfaces/IBuildPatchServicesModule.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "ToolModes/ToolModesHelp.h"

using namespace BuildPatchTool;

class FEnumerationToolMode : public IToolMode
{
public:
	FEnumerationToolMode(IBuildPatchServicesModule& InBpsInterface, const TCHAR* InCommandLine)
		: BpsInterface(InBpsInterface)
		, CommandLine(InCommandLine)
	{}

	virtual ~FEnumerationToolMode()
	{}

	virtual EReturnCode Execute() override
	{
		using namespace BuildPatchServices;

		// Parse commandline
		if (ProcessCommandline() == false)
		{
			return EReturnCode::ArgumentProcessingError;
		}

		// Print help if requested
		if (bHelp)
		{
			PrintHelp<FEnumerationToolModeHelp>();
			return EReturnCode::OK;
		}

		LogConfig();

		// Register secrets.
		for (const TPair<FGuid, TArray<uint8>>& IdKeyPair : EncryptionSecrets)
		{
			BpsInterface.RegisterEncryptionSecret(IdKeyPair.Key, IdKeyPair.Value);
		}

		// Check existence of input file
		if (!FPaths::FileExists(InputFile))
		{
			UE_LOGF(LogBuildPatchTool, Error, "Provided input file was not found: %ls", *InputFile);
			return EReturnCode::FileNotFound;
		}

		// Setup and run.
		FPatchDataEnumerationConfiguration Settings;
		Settings.InputFile = InputFile;
		Settings.OutputFile = OutputFile;
		Settings.bIncludeSizes = bIncludeSizes;

		// Run the enumeration routine
		IPatchDataEnumerationRef PatchDataEnumeration = BpsInterface.CreatePatchDataEnumeration(Settings);
		const bool bSuccess = PatchDataEnumeration->Run();
		return bSuccess ? EReturnCode::OK : EReturnCode::ToolFailure;
	}

private:

	bool ProcessCommandline()
	{
#define PARSE_SWITCH(Switch) ParseSwitch(TEXT(#Switch "="), Switch, Switches)
		TArray<FString> Tokens, Switches;
		FCommandLine::Parse(CommandLine, Tokens, Switches);

		bHelp = ParseOption(TEXT("help"), Switches);
		if (bHelp)
		{
			return true;
		}

		// Get all required parameters
		if (!PARSE_SWITCH(InputFile) || InputFile.IsEmpty()
		  || !PARSE_SWITCH(OutputFile) || OutputFile.IsEmpty())
		{
			UE_LOGF(LogBuildPatchTool, Error, "InputFile and OutputFile are required parameters");
			return false;
		}

		if (InputFile.Equals(OutputFile, ESearchCase::IgnoreCase))
		{
			UE_LOGF(LogBuildPatchTool, Error, "InputFile and OutputFile cannot be the same");
			return false;
		}

		NormalizeUriFile(InputFile);
		NormalizeUriFile(OutputFile);

		// Get optional parameters
		bIncludeSizes = ParseOption(TEXT("includesizes"), Switches);

		// Parse all the secret key and ID pairs.
		FString ParseErrorMessage;
		if (!ParsePairs(TEXT("EncryptionSecretId="), TEXT("EncryptionSecretKey="), Switches, EncryptionSecrets, ParseErrorMessage) && !ParseErrorMessage.IsEmpty())
		{
			UE_LOGF(LogBuildPatchTool, Error, "%ls", *ParseErrorMessage);
			return false;
		}

		return true;
#undef PARSE_SWITCH
	}

	void LogConfig() const
	{
		UE_LOGF(LogBuildPatchTool, Log, "-----Configuration for EnumerationMode------");
		UE_LOGF(LogBuildPatchTool, Log, "   InputFile: %ls", *InputFile);
		UE_LOGF(LogBuildPatchTool, Log, "   OutputFile: %ls", *OutputFile);
		UE_LOGF(LogBuildPatchTool, Log, "   bIncludeSizes: %ls", bIncludeSizes ? TEXT("true") : TEXT("false"));
		for (const TPair<FGuid, TArray<uint8>>& EncryptionSecret : EncryptionSecrets)
		{
			UE_LOGF(LogBuildPatchTool, Log, "   AvailableSecretId: %ls", *EncryptionSecret.Key.ToString());
		}
	}

private:
	static TCHAR const* const MODE_NAME;
	IBuildPatchServicesModule& BpsInterface;
	const TCHAR* CommandLine;
	bool bHelp;
	FString InputFile;
	FString OutputFile;
	bool bIncludeSizes;
	TMap<FGuid, TArray<uint8>> EncryptionSecrets;
};

IMPLEMENT_BPT_MODE(Enumeration, FEnumerationToolMode);
