// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolModes/ExtractMetadataMode.h"

#include "Async/Async.h"
#include "BuildPatchFeatureLevel.h"
#include "BuildPatchTool.h"
#include "Common/MetadataSerialiser.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Interfaces/IBuildPatchServicesModule.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/Paths.h"
#include "ToolModes/ToolModesHelp.h"

using namespace BuildPatchTool;

class FExtractMetadataToolMode : public IToolMode
{
public:
	FExtractMetadataToolMode(IBuildPatchServicesModule& InBpsInterface, const TCHAR* InCommandLine)
		: BpsInterface(InBpsInterface)
		, CommandLine(InCommandLine)
		, bHelp(false)
		, bOpenOutputFiles(false)
		, OutputFormat(EMetadataOutputFormat::Json)
	{}

	virtual ~FExtractMetadataToolMode()
	{}

	virtual EReturnCode Execute() override
	{
		// Parse commandline
		if (ProcessCommandline() == false)
		{
			return EReturnCode::ArgumentProcessingError;
		}

		// Print help if requested
		if (bHelp)
		{
			PrintHelp<FExtractMetadataToolModeHelp>();
			return EReturnCode::OK;
		}

		// Register secrets.
		for (const TPair<FGuid, TArray<uint8>>& IdKeyPair : EncryptionSecrets)
		{
			BpsInterface.RegisterEncryptionSecret(IdKeyPair.Key, IdKeyPair.Value);
		}

		bool bSuccess = true;

		TArray<TFuture<bool>> FileFutures;

		// Process each file.
		for (const TPair<FString, FString>& InputOutputPair : InputOutputPairs)
		{
			const FString& InputFilename = InputOutputPair.Get<0>();
			const FString& OutputFilename = InputOutputPair.Get<1>();

			TFunction<bool()> TaskManifestA = [this, InputFilename, OutputFilename]()
			{
				bool bResult = true;
				// Is this a manifest?
				if (InputFilename.EndsWith(TEXT(".manifest")) || InputFilename.EndsWith(TEXT(".delta")))
				{
					IBuildManifestPtr Manifest = BpsInterface.LoadManifestFromFile(InputFilename);
					if (Manifest.IsValid())
					{
						// Serialise the output into desired string format
						FString OutputString = FMetadataSerialiser::SerialiseMetadata(*Manifest.Get(), OutputFormat);
						// Save the output
						FFileHelper::SaveStringToFile(OutputString, *OutputFilename);
						UE_LOGF(LogBuildPatchTool, Display, "Manifest meta saved: %ls", *OutputFilename);
						// Open the output
						if (bOpenOutputFiles)
						{
							FPlatformProcess::LaunchFileInDefaultExternalApplication(*OutputFilename);
						}
					}
					else
					{
						UE_LOGF(LogBuildPatchTool, Error, "Failed to read file: %ls", *InputFilename);
						bResult = false;
					}
				}
				else
				{
					UE_LOGF(LogBuildPatchTool, Warning, "Skipping unknown file type: %ls", *FPaths::GetExtension(InputFilename));
				}
				return bResult;
			};
			FileFutures.Add(Async(EAsyncExecution::ThreadPool, MoveTemp(TaskManifestA)));
		}

		// Wait on all futures.
		for (const TFuture<bool>& FileFuture : FileFutures)
		{
			bSuccess = FileFuture.Get() && bSuccess;
			GLog->FlushThreadedLogs();
		}

		return bSuccess ? EReturnCode::OK : EReturnCode::ToolFailure;
	}

private:

	bool ProcessCommandline()
	{
		TArray<FString> Tokens, Switches;
		FCommandLine::Parse(CommandLine, Tokens, Switches);

		bHelp = ParseOption(TEXT("help"), Switches);
		if (bHelp)
		{
			return true;
		}

		// Check for output format switch
		FString OutputFormatString;
		if (ParseSwitch(TEXT("OutputFormat="), OutputFormatString, Switches))
		{
			LexFromString(OutputFormat, *OutputFormatString);
			if (OutputFormat == EMetadataOutputFormat::InvalidOrMax)
			{
				UE_LOGF(LogBuildPatchTool, Error, "Invalid arg: -OutputFormat=%ls. OutputFormat was provided, but not recognized.", *OutputFormatString);
				return false;
			}
		}
		bOpenOutputFiles = ParseOption(TEXT("OpenOutput"), Switches);

		// We need to parse all the input, output pairs. We expect every -InputFile= to be followed by a matching -OutputFile=
		for (int32 SwitchIndex = 0; SwitchIndex < Switches.Num()-1; SwitchIndex++)
		{
			const FString& InputSwitch = Switches[SwitchIndex];
			const FString& OutputSwitch = Switches[SwitchIndex+1];
			// Check for param name
			if (InputSwitch.StartsWith(TEXT("InputFile=")))
			{
				if (OutputSwitch.StartsWith(TEXT("OutputFile=")))
				{
					TPair<FString, FString> InputOutputPair;
					InputSwitch.Split(IToolMode::EqualsStr, nullptr, &InputOutputPair.Get<0>());
					OutputSwitch.Split(IToolMode::EqualsStr, nullptr, &InputOutputPair.Get<1>());
					InputOutputPair.Get<0>() = InputOutputPair.Get<0>().TrimQuotes();
					InputOutputPair.Get<1>() = InputOutputPair.Get<1>().TrimQuotes();
					NormalizeUriFile(InputOutputPair.Get<0>());
					NormalizeUriFile(InputOutputPair.Get<1>());

					if (InputOutputPair.Get<0>().IsEmpty() || InputOutputPair.Get<1>().IsEmpty())
					{
						UE_LOGF(LogBuildPatchTool, Error, "InputFile and OutputFile arguments can't be empty");
						return false;
					}

					InputOutputPairs.Add(MoveTemp(InputOutputPair));
				}
				else
				{
					// FAIL
					UE_LOGF(LogBuildPatchTool, Error, "InputFile and OutputFile parameters must be provided as pairs.");
					return false;
				}
			}
		}

		if (InputOutputPairs.Num() < 1)
		{
			UE_LOGF(LogBuildPatchTool, Error, "At least one InputFile/OutputFile pair must be specified");
			return false;
		}

		// Parse all the secret key and ID pairs.
		FString ParseErrorMessage;
		if (!ParsePairs(TEXT("EncryptionSecretId="), TEXT("EncryptionSecretKey="), Switches, EncryptionSecrets, ParseErrorMessage) && !ParseErrorMessage.IsEmpty())
		{
			UE_LOGF(LogBuildPatchTool, Error, "%ls", *ParseErrorMessage);
			return false;
		}

		return true;
	}

private:
	static TCHAR const* const MODE_NAME;
	IBuildPatchServicesModule& BpsInterface;
	const TCHAR* CommandLine;
	bool bHelp;
	bool bOpenOutputFiles;
	EMetadataOutputFormat OutputFormat;
	TArray<TPair<FString, FString>> InputOutputPairs;
	TMap<FGuid, TArray<uint8>> EncryptionSecrets;
};

IMPLEMENT_BPT_MODE(ExtractMetadata, FExtractMetadataToolMode);
