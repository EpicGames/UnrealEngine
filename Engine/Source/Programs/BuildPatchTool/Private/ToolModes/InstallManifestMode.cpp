// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolModes/InstallManifestMode.h"

#include "Algo/Transform.h"
#include "Async/TaskGraphInterfaces.h"
#include "BuildPatchTool.h"
#include "Containers/Ticker.h"
#include "HttpModule.h"
#include "Interfaces/IHttpResponse.h"
#include "Interfaces/ToolMode.h"
#include "Interfaces/IBuildPatchServicesModule.h"
#include "Misc/CommandLine.h"
#include "Misc/Optional.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/Paths.h"
#include "ToolModes/ToolModesHelp.h"

class FInstallManifestToolMode : public BuildPatchTool::IToolMode
{
public:
	FInstallManifestToolMode(IBuildPatchServicesModule& BpsInterface, const TCHAR* CommandLine);
	virtual BuildPatchTool::EReturnCode Execute() override;

private:
	void RequestManifestFile(const FString& ManifestUri);
	void AppManifestHttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, FString ManifestUri);
	void FilterInstallation();
	void StartInstallation();
	void OnInstallComplete(const IBuildInstallerRef& Installer);
	void ExitTool(BuildPatchTool::EReturnCode InReturnCode);
	bool ProcessCommandline();
	void LogConfig() const;

private:
	static TCHAR const* const MODE_NAME;
	IBuildPatchServicesModule& BpsInterface;
	const TCHAR* CommandLine;
	BuildPatchTool::EReturnCode ReturnCode;
	bool bHelp;
	TArray<FInstallActionArgs> InstallActions;
	FString OutputDir;
	TSet<FString> FileList;
	TArray<FString> CloudDirs;
	TMap<FGuid, TArray<uint8>> EncryptionSecrets;
	TMap<FString, IBuildManifestPtr> ActionManifests;
	IBuildInstallerPtr BuildInstaller;
	bool bRejectSymlinks = false;
};

FInstallManifestToolMode::FInstallManifestToolMode(IBuildPatchServicesModule& InBpsInterface, const TCHAR* InCommandLine)
	: BpsInterface(InBpsInterface)
	, CommandLine(InCommandLine)
	, ReturnCode(BuildPatchTool::EReturnCode::OK)
	, bHelp(false)
{
}

BuildPatchTool::EReturnCode FInstallManifestToolMode::Execute()
{
	// Run any core initialisation required.
	FHttpModule::Get();

	// Parse commandline.
	if (ProcessCommandline() == false)
	{
		return BuildPatchTool::EReturnCode::ArgumentProcessingError;
	}

	// Print help if requested.
	if (bHelp)
	{
		PrintHelp<BuildPatchTool::FInstallManifestToolModeHelp>();
		return BuildPatchTool::EReturnCode::OK;
	}
	LogConfig();

	// Register secrets.
	for (const TPair<FGuid, TArray<uint8>>& IdKeyPair : EncryptionSecrets)
	{
		BpsInterface.RegisterEncryptionSecret(IdKeyPair.Key, IdKeyPair.Value);
	}

	// Main timers.
	double DeltaTime = 0.0;
	double LastTime = FPlatformTime::Seconds();

	// Setup desired frame times.
	const float MainsFramerate = 100.0f;
	const float MainsFrameTime = 1.0f / MainsFramerate;

	// Setup and kick off all relevant manifest loads.
	for (const FInstallActionArgs& InstallAction : InstallActions)
	{
		if (InstallAction.PrevManifestUri.IsSet())
		{
			ActionManifests.Add(InstallAction.PrevManifestUri.GetValue(), IBuildManifestPtr());
		}
		if (InstallAction.NewManifestUri.IsSet())
		{
			ActionManifests.Add(InstallAction.NewManifestUri.GetValue(), IBuildManifestPtr());
		}
	}
	for (TPair<FString, IBuildManifestPtr>& ActionManifest : ActionManifests)
	{
		RequestManifestFile(ActionManifest.Key);
	}

	// Run the tick loop.
	while (!IsEngineExitRequested())
	{
		// Increment global frame counter once for each app tick.
		GFrameCounter++;

		// Application tick.
		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
		FTSTicker::GetCoreTicker().Tick(DeltaTime);

		// Flush logs.
		GLog->FlushThreadedLogs();

		// Control frame rate.
		FPlatformProcess::Sleep(FMath::Max<float>(0.0f, MainsFrameTime - (FPlatformTime::Seconds() - LastTime)));

		// Calculate deltas.
		const double AppTime = FPlatformTime::Seconds();
		DeltaTime = AppTime - LastTime;
		LastTime = AppTime;
	}

	return ReturnCode;
}

void FInstallManifestToolMode::RequestManifestFile(const FString& ManifestUri)
{
	const bool bIsHttpRequest = ManifestUri.StartsWith(TEXT("http"), ESearchCase::IgnoreCase);
	if (bIsHttpRequest)
	{
		// Create a HTTP request.
		TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();
		HttpRequest->OnProcessRequestComplete().BindRaw(this, &FInstallManifestToolMode::AppManifestHttpRequestComplete, ManifestUri);
		HttpRequest->SetURL(ManifestUri);
		HttpRequest->SetVerb(TEXT("GET"));
		if (!HttpRequest->ProcessRequest())
		{
			UE_LOGF(LogBuildPatchTool, Error, "Unable to process manifest http request %ls", *ManifestUri);
			ExitTool(BuildPatchTool::EReturnCode::ToolFailure);
		}
	}
	else
	{
		// Read direct from file.
		IBuildManifestPtr* ManifestToLoad = ActionManifests.Find(ManifestUri);
		if (ManifestToLoad == nullptr)
		{
			UE_LOGF(LogBuildPatchTool, Error, "Missing ActionManifest request %ls", *ManifestUri);
			ExitTool(BuildPatchTool::EReturnCode::ToolFailure);
		}
		else
		{
			*ManifestToLoad = BpsInterface.LoadManifestFromFile(ManifestUri);
			if (!ManifestToLoad->IsValid())
			{
				UE_LOGF(LogBuildPatchTool, Error, "Unable to read manifest from %ls", *ManifestUri);
				ExitTool(BuildPatchTool::EReturnCode::InvalidData);
			}
			else
			{
				FilterInstallation();
			}
		}
	}
}

void FInstallManifestToolMode::AppManifestHttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, FString ManifestUri)
{
	if (bSucceeded && HttpResponse.IsValid())
	{
		IBuildManifestPtr* ManifestToLoad = ActionManifests.Find(ManifestUri);
		if (ManifestToLoad == nullptr)
		{
			UE_LOGF(LogBuildPatchTool, Error, "Missing ActionManifest request %ls", *ManifestUri);
			ExitTool(BuildPatchTool::EReturnCode::ToolFailure);
		}
		else
		{
			*ManifestToLoad = BpsInterface.MakeManifestFromData(HttpResponse->GetContent());
			if (!ManifestToLoad->IsValid())
			{
				UE_LOGF(LogBuildPatchTool, Error, "Unable to serialise manifest from %ls", *HttpRequest->GetURL());
				ExitTool(BuildPatchTool::EReturnCode::InvalidData);
			}
			else
			{
				FilterInstallation();
			}
		}
	}
	else
	{
		UE_LOGF(LogBuildPatchTool, Error, "Unable to download manifest from %ls", *HttpRequest->GetURL());
		ExitTool(BuildPatchTool::EReturnCode::InvalidData);
	}
}

void FInstallManifestToolMode::FilterInstallation()
{
	// Ignore until we have loaded all manifest.
	const bool bLoadComplete = Algo::AllOf(ActionManifests, [](const TPair<FString, IBuildManifestPtr>& Elem) { return Elem.Value.IsValid(); });
	if (!bLoadComplete)
	{
		return;
	}

	// Filter all manifests by provided file list.
	if (FileList.Num() > 0)
	{
		TSet<FString> ResultingFileList;
		for (TPair<FString, IBuildManifestPtr>& ActionManifest : ActionManifests)
		{
			IBuildManifestPtr& FilteredManifest = ActionManifest.Value;
			TSet<FString> FilteredFileList = FileList.Intersect(TSet<FString>(FilteredManifest->GetBuildFileList()));
			ResultingFileList.Append(FilteredFileList);
			FilteredManifest = BpsInterface.MergeManifests(FilteredManifest.ToSharedRef(), FilteredManifest.ToSharedRef(), FilteredManifest->GetVersionString(), TSet<FString>(), FilteredFileList);
			if (!FilteredManifest.IsValid())
			{
				UE_LOGF(LogBuildPatchTool, Error, "Manifest filtering failed, check file list and encryption secrets.");
				ExitTool(BuildPatchTool::EReturnCode::ArgumentProcessingError);
				return;
			}
		}
		// Check that all files in FileList were indeed found in one of the actions
		// MergeManifests will not fail because we are filtering the FileList input by action  manifest in order
		// to support both multiple actions and a FileList together
		bool bAllFilesFound = true;
		for (const FString& File : FileList)
		{
			if (!ResultingFileList.Contains(File))
			{
				UE_LOGF(LogBuildPatchTool, Error, "Could not find file in FileList: %ls", *File);
				bAllFilesFound = false;
			}
		}
		if (!bAllFilesFound)
		{
			// Error is logged above.
			ExitTool(BuildPatchTool::EReturnCode::ArgumentProcessingError);
			return;
		}
	}

	StartInstallation();
}

void FInstallManifestToolMode::StartInstallation()
{
	using namespace BuildPatchServices;

	// Enumerate secrets.
	TSet<FGuid> AvailableSecrets;
	TSet<FGuid> NecessaryEncryptionSecrets;
	Algo::Transform(EncryptionSecrets, AvailableSecrets, &TPair<FGuid, TArray<uint8>>::Key);
	for (TPair<FString, IBuildManifestPtr>& ActionManifest : ActionManifests)
	{
		NecessaryEncryptionSecrets.Append(ActionManifest.Value->GetNecessaryEncryptionSecretIds());
	}

	// Configure the installer actions.
	TArray<FInstallerAction> InstallerActions;
	for (const FInstallActionArgs& InstallAction : InstallActions)
	{
		// ActionManifests has been previously validated and the tool exits if we are missing entries that are required,
		// or if any uris failed to serialise into a valid manifest.

		// Uninstall if NewManifestUri was not provided.
		if (!InstallAction.NewManifestUri.IsSet())
		{
			InstallerActions.Add(FInstallerAction::MakeUninstall(ActionManifests.FindRef(InstallAction.PrevManifestUri.GetValue()).ToSharedRef()));
		}
		// Install if PrevManifestUri was not prvidedo.
		else if (!InstallAction.PrevManifestUri.IsSet())
		{
			InstallerActions.Add(FInstallerAction::MakeInstall(ActionManifests.FindRef(InstallAction.NewManifestUri.GetValue()).ToSharedRef(), InstallAction.InstallTags.GetValue()));
		}
		// Otherwise, it's an update.
		else
		{
			InstallerActions.Add(FInstallerAction::MakeUpdate(ActionManifests.FindRef(InstallAction.PrevManifestUri.GetValue()).ToSharedRef(), ActionManifests.FindRef(InstallAction.NewManifestUri.GetValue()).ToSharedRef(), InstallAction.InstallTags.GetValue()));
		}
	}

	// Configure the installer.
	FBuildInstallerConfiguration Configuration(MoveTemp(InstallerActions));
	Configuration.InstallDirectory = OutputDir;
	Configuration.StagingDirectory = FPaths::Combine(OutputDir, TEXT(".bptstage"));
	Configuration.CloudDirectories = CloudDirs;
	Configuration.EncryptionSecrets.Append(EncryptionSecrets);
	Configuration.bRejectSymlinks = bRejectSymlinks;
	if (NecessaryEncryptionSecrets.Difference(AvailableSecrets).Num() > 0)
	{
		Configuration.InstallMode = EInstallMode::Preload;
	}
	// Minimal verification
	Configuration.VerifyMode = EVerifyMode::FileSizeCheckTouchedFiles;

	// Start the installer.
	FBuildPatchInstallerDelegate InstallerCompleteDelegate = FBuildPatchInstallerDelegate::CreateRaw(this, &FInstallManifestToolMode::OnInstallComplete);
	BuildInstaller = BpsInterface.CreateBuildInstaller(Configuration, MoveTemp(InstallerCompleteDelegate));
	BuildInstaller->StartInstallation();

}

void FInstallManifestToolMode::OnInstallComplete(const IBuildInstallerRef& Installer)
{
	using namespace BuildPatchServices;
	const FBuildInstallerConfiguration& InstallerConfiguration = Installer->GetConfiguration();
	const FBuildInstallStats& InstallBuildStatistics = Installer->GetBuildStatistics();
	const bool bSuccess = Installer->CompletedSuccessfully();

	if (!bSuccess)
	{
		UE_LOGF(LogBuildPatchTool, Display, "Install completed with failure. Error code: %ls.", *Installer->GetErrorCode());
		ExitTool(BuildPatchTool::EReturnCode::ToolFailure);
	}
	else
	{
		UE_LOGF(LogBuildPatchTool, Display, "Install completed successfully.");
		ExitTool(BuildPatchTool::EReturnCode::OK);
	}
}

void FInstallManifestToolMode::ExitTool(BuildPatchTool::EReturnCode InReturnCode)
{
	ReturnCode = InReturnCode;
	RequestEngineExit(TEXT("Installer complete."));
}

bool FInstallManifestToolMode::ProcessCommandline()
{
#define PARSE_SWITCH(Switch) ParseSwitch(TEXT(#Switch "="), Switch, Switches)

	TArray<FString> Tokens, Switches;
	FCommandLine::Parse(CommandLine, Tokens, Switches);
	FString ParseErrorMessage;

	bHelp = ParseOption(TEXT("help"), Switches);
	if (bHelp)
	{
		return true;
	}

	// Parse required install actions
	if (!ParseInstallActions(Switches, InstallActions, ParseErrorMessage))
	{
		UE_LOGF(LogBuildPatchTool, Error, "%ls", *ParseErrorMessage);
		return false;
	}

	PARSE_SWITCH(OutputDir);
	PARSE_SWITCH(FileList);
	PARSE_SWITCH(CloudDirs);

	bRejectSymlinks = ParseOption(TEXT("RejectSymlinks"), Switches);

	NormalizeUriPath(OutputDir);
	for (FString& File : FileList)
	{
		NormalizeUriPath(File);
	}
	for (FString& CloudDir : CloudDirs)
	{
		NormalizeUriFile(CloudDir);
	}

	// If specific cloud dirs were not provided, we take the paths of the manifests in the actions we found.
	if (CloudDirs.Num() == 0)
	{
		for (const FInstallActionArgs& InstallAction : InstallActions)
		{
			if (InstallAction.PrevManifestUri.IsSet())
			{
				CloudDirs.AddUnique(FPaths::GetPath(InstallAction.PrevManifestUri.GetValue()));
			}
			if (InstallAction.NewManifestUri.IsSet())
			{
				CloudDirs.AddUnique(FPaths::GetPath(InstallAction.NewManifestUri.GetValue()));
			}
		}
	}

	// Parse all the secret key and ID pairs.
	if (!ParsePairs(TEXT("EncryptionSecretId="), TEXT("EncryptionSecretKey="), Switches, EncryptionSecrets, ParseErrorMessage) && !ParseErrorMessage.IsEmpty())
	{
		UE_LOGF(LogBuildPatchTool, Error, "%ls", *ParseErrorMessage);
		return false;
	}

#undef PARSE_SWITCH
	return true;
}

void FInstallManifestToolMode::LogConfig() const
{
	UE_LOGF(LogBuildPatchTool, Log, "-----Configuration for InstallManifest------");
	for (const FInstallActionArgs& InstallAction : InstallActions)
	{
		if (InstallAction.PrevManifestUri.IsSet())
		{
			UE_LOGF(LogBuildPatchTool, Log, "  PrevManifest:   %ls", *InstallAction.PrevManifestUri.GetValue());
		}
		if (InstallAction.NewManifestUri.IsSet())
		{
			UE_LOGF(LogBuildPatchTool, Log, "  NewManifest:    %ls", *InstallAction.NewManifestUri.GetValue());
		}
		if (InstallAction.InstallTags.IsSet())
		{
			for (const FString& Tag : InstallAction.InstallTags.GetValue())
			{
				UE_LOGF(LogBuildPatchTool, Log, "  InstallTag:     %ls", *Tag);
			}
		}
	}

	UE_LOGF(LogBuildPatchTool, Log, "  RejectSymlinks: %ls", bRejectSymlinks ? TEXT("true") : TEXT("false"));
	UE_LOGF(LogBuildPatchTool, Log, "  OutputDir:      %ls", *OutputDir);
	bool bFirst = true;
	for (const FString& File : FileList)
	{
		if (bFirst)
		{
			bFirst = false;
			UE_LOGF(LogBuildPatchTool, Log, "  FileList:     %ls", *File);
		}
		else
		{
			UE_LOGF(LogBuildPatchTool, Log, "                %ls", *File);
		}
	}
	bFirst = true;
	for (const FString& CloudDir : CloudDirs)
	{
		if (bFirst)
		{
			bFirst = false;
			UE_LOGF(LogBuildPatchTool, Log, "  CloudDirs:      %ls", *CloudDir);
		}
		else
		{
			UE_LOGF(LogBuildPatchTool, Log, "                  %ls", *CloudDir);
		}
	}
	for (const TPair<FGuid, TArray<uint8>>& EncryptionSecret : EncryptionSecrets)
	{
		UE_LOGF(LogBuildPatchTool, Log, "  AvailableSecretId: %ls", *EncryptionSecret.Key.ToString());
	}
}

IMPLEMENT_BPT_MODE(InstallManifest, FInstallManifestToolMode);
