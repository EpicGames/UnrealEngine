// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolModes/MergeManifestMode.h"

#include "BuildPatchTool.h"
#include "Interfaces/IBuildPatchServicesModule.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "ToolModes/ToolModesHelp.h"

using namespace BuildPatchTool;

class FMergeManifestToolMode : public IToolMode
{
public:
	FMergeManifestToolMode(IBuildPatchServicesModule& InBpsInterface, const TCHAR* InCommandLine)
		: BpsInterface(InBpsInterface)
		, CommandLine(InCommandLine)
	{}

	virtual ~FMergeManifestToolMode()
	{}

	virtual EReturnCode Execute() override
	{
		// Parse commandline
		if (ProcessCommandline() == false)
		{
			return EReturnCode::ArgumentProcessingError;
		}

		// Print help if requested
		if (bHelp)
		{
			PrintHelp<FMergeManifestToolModeHelp>();
			return EReturnCode::OK;
		}

		LogConfig();

		// Register secrets.
		for (const TPair<FGuid, TArray<uint8>>& IdKeyPair : EncryptionSecrets)
		{
			BpsInterface.RegisterEncryptionSecret(IdKeyPair.Key, IdKeyPair.Value);
		}

		// Run the merge manifest routine
		bool bSuccess = BpsInterface.MergeManifests(ManifestA, ManifestB, ManifestC, BuildVersion, MergeFileList);
		return bSuccess ? EReturnCode::OK : EReturnCode::ToolFailure;
	}

private:

	bool ProcessCommandline()
	{
#define PARSE_SWITCH(Switch) ParseSwitch(TEXT(#Switch "="), Switch, Switches)
		TArray<FString> Tokens, Switches;
		FCommandLine::Parse(CommandLine, Tokens, Switches);

		bHelp = ParseOption(TEXT("help"), Switches);
		if (bHelp)
		{
			return true;
		}

		// Get all required parameters
		if (!(PARSE_SWITCH(ManifestA)
		   && PARSE_SWITCH(ManifestB)
		   && PARSE_SWITCH(ManifestC)
		   && PARSE_SWITCH(BuildVersion)))
		{
			UE_LOGF(LogBuildPatchTool, Error, "ManifestA, ManifestB, ManifestC, and BuildVersion are required parameters");
			return false;
		}

		if (ManifestA.IsEmpty() || ManifestB.IsEmpty() || ManifestC.IsEmpty() || BuildVersion.IsEmpty())
		{
			UE_LOGF(LogBuildPatchTool, Error, "ManifestA/B/C and BuildVersion arguments can't be empty");
			return false;
		}

		NormalizeUriFile(ManifestA);
		NormalizeUriFile(ManifestB);
		NormalizeUriFile(ManifestC);

		if (ManifestC == ManifestA || ManifestC == ManifestB)
		{
			UE_LOGF(LogBuildPatchTool, Error, "ManifestC parameter can't point to the same file as ManifestA or ManifestB");
			return false;
		}

		// Optional list to pick specific files, otherwise it is A stomped by B
		PARSE_SWITCH(MergeFileList);
		NormalizeUriFile(MergeFileList);

		// Parse all the secret key and ID pairs.
		FString ParseErrorMessage;
		if (!ParsePairs(TEXT("EncryptionSecretId="), TEXT("EncryptionSecretKey="), Switches, EncryptionSecrets, ParseErrorMessage) && !ParseErrorMessage.IsEmpty())
		{
			UE_LOGF(LogBuildPatchTool, Error, "%ls", *ParseErrorMessage);
			return false;
		}

		return true;
#undef PARSE_SWITCH
	}

	void LogConfig() const
	{
		UE_LOGF(LogBuildPatchTool, Log, "-----Configuration for MergeManifestMode------");
		UE_LOGF(LogBuildPatchTool, Log, "   ManifestA: %ls", *ManifestA);
		UE_LOGF(LogBuildPatchTool, Log, "   ManifestB: %ls", *ManifestB);
		UE_LOGF(LogBuildPatchTool, Log, "   ManifestC: %ls", *ManifestC);
		UE_LOGF(LogBuildPatchTool, Log, "   BuildVersion: %ls", *BuildVersion);
		UE_LOGF(LogBuildPatchTool, Log, "   MergeFileList: %ls", *MergeFileList);
		for (const TPair<FGuid, TArray<uint8>>& EncryptionSecret : EncryptionSecrets)
		{
			UE_LOGF(LogBuildPatchTool, Log, "   AvailableSecretId: %ls", *EncryptionSecret.Key.ToString());
		}
	}

private:
	static TCHAR const* const MODE_NAME;
	IBuildPatchServicesModule& BpsInterface;
	const TCHAR* CommandLine;
	bool bHelp;
	FString ManifestA;
	FString ManifestB;
	FString ManifestC;
	FString BuildVersion;
	FString MergeFileList;
	TMap<FGuid, TArray<uint8>> EncryptionSecrets;
};

IMPLEMENT_BPT_MODE(MergeManifests, FMergeManifestToolMode);
