// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolModes/PackageChunksMode.h"

#include "Algo/Find.h"
#include "Algo/Count.h"
#include "BuildPatchTool.h"
#include "Interfaces/IBuildPatchServicesModule.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "ToolModes/ToolModesHelp.h"

using namespace BuildPatchTool;

class FPackageChunksToolMode : public IToolMode
{
public:
	FPackageChunksToolMode(IBuildPatchServicesModule& InBpsInterface, const TCHAR* InCommandLine)
		: BpsInterface(InBpsInterface)
		, CommandLine(InCommandLine)
	{}

	virtual ~FPackageChunksToolMode()
	{}

	virtual EReturnCode Execute() override
	{
		// Parse commandline.
		if (ProcessCommandline() == false)
		{
			return EReturnCode::ArgumentProcessingError;
		}

		// Print help if requested.
		if (bHelp)
		{
			PrintHelp<FPackageChunksToolModeHelp>();
			return EReturnCode::OK;
		}

		LogConfig();

		// Register secrets.
		for (const TPair<FGuid, TArray<uint8>>& IdKeyPair : EncryptionSecrets)
		{
			BpsInterface.RegisterEncryptionSecret(IdKeyPair.Key, IdKeyPair.Value);
		}

		// Setup and run.
		BuildPatchServices::FPackageChunksConfiguration Configuration;
		LexFromString(Configuration.FeatureLevel, *FeatureLevel);
		if (Configuration.FeatureLevel == BuildPatchServices::EFeatureLevel::Invalid)
		{
			UE_LOGF(LogBuildPatchTool, Error, "Provided FeatureLevel is not recognised. Invalid arg: -FeatureLevel=%ls", *FeatureLevel);
			return EReturnCode::ArgumentProcessingError;
		}
		Configuration.ManifestFilePath = ManifestFile;
		Configuration.PrevManifestFilePath = PrevManifestFile;
		Configuration.TagSetArray = TagSetArray;
		Configuration.PrevTagSet = PrevTagSet;
		Configuration.OutputFile = OutputFile;
		Configuration.CloudDir = CloudDir;
		Configuration.MaxOutputFileSize = MaxOutputFileSize;
		Configuration.ResultDataFilePath = ResultDataFile;
		Configuration.bIgnoreManifestFileInvalidTags = bIgnoreManifestFileInvalidTags;
		Configuration.bIgnorePrevManifestFileInvalidTags = bIgnorePrevManifestFileInvalidTags;
		Configuration.DeltaFilenameTrailer = DeltaFilenameTrailer;

		// Run the enumeration routine.
		bool bSuccess = BpsInterface.PackageChunkData(Configuration);
		return bSuccess ? EReturnCode::OK : EReturnCode::ToolFailure;
	}

private:

	bool ProcessCommandline()
	{
#define HAS_SWITCH(SwitchVar) (Algo::FindByPredicate(Switches, [](const FString& Elem){ return Elem.StartsWith(TEXT(#SwitchVar "="));}) != nullptr)
#define HAS_DUP_SWITCH(SwitchVar) (Algo::CountIf(Switches, [](const FString& Elem){ return Elem.StartsWith(TEXT(#SwitchVar "="));}) > 1)
#define PARSE_SWITCH(SwitchVar) ParseSwitch(TEXT(#SwitchVar "="), SwitchVar, Switches)
#define PARSE_SWITCHES(SwitchesVar) ParseSwitches(TEXT(#SwitchesVar "="), SwitchesVar, Switches)
		TArray<FString> Tokens, Switches;
		FCommandLine::Parse(CommandLine, Tokens, Switches);

		bHelp = ParseOption(TEXT("help"), Switches);
		if (bHelp)
		{
			return true;
		}

		// Grab the FeatureLevel. This is required param but safe to default, we can change this to a warning after first release, and then an error later, as part of a friendly roll out.
		PARSE_SWITCH(FeatureLevel);
		FeatureLevel.TrimStartAndEndInline();
		if (FeatureLevel.IsEmpty())
		{
			UE_LOGF(LogBuildPatchTool, Log, "FeatureLevel was not provided, defaulting to LatestJson. Please provide the FeatureLevel commandline argument which matches the existing client support.");
			FeatureLevel = TEXT("LatestJson");
		}

		// Get all required parameters.
		if (!(PARSE_SWITCH(ManifestFile)
		   && PARSE_SWITCH(OutputFile)))
		{
			UE_LOGF(LogBuildPatchTool, Error, "ManifestFile and OutputFile are required parameters");
			return false;
		}

		if (ManifestFile.IsEmpty())
		{
			UE_LOGF(LogBuildPatchTool, Error, "ManifestFile argument can't be empty");
			return false;
		}

		if (OutputFile.IsEmpty())
		{
			UE_LOGF(LogBuildPatchTool, Error, "OutputFile argument can't be empty");
			return false;
		}

		NormalizeUriFile(ManifestFile);
		NormalizeUriFile(OutputFile);

		// Get optional parameters.
		PARSE_SWITCH(PrevManifestFile);
		PARSE_SWITCH(ResultDataFile);
		NormalizeUriFile(PrevManifestFile);
		NormalizeUriFile(ResultDataFile);
		TArray<FString> TagSets;
		PARSE_SWITCHES(TagSets);
		PARSE_SWITCH(PrevTagSet);
		PARSE_SWITCH(DeltaFilenameTrailer);

		if (HAS_DUP_SWITCH(PrevTagSet))
		{
			UE_LOGF(LogBuildPatchTool, Error, "Only one PrevTagSet should be provided");
			return false;
		}

		if (ManifestFile == PrevManifestFile)
		{
			UE_LOGF(LogBuildPatchTool, Error, "ManifestFile and PreManifestFile should not point to the same file");
			return false;
		}

		if (!PARSE_SWITCH(CloudDir))
		{
			// If not provided we use the location of the manifest file.
			CloudDir = FPaths::GetPath(ManifestFile);
		}
		NormalizeUriPath(CloudDir);

		if (HAS_SWITCH(MaxOutputFileSize))
		{
			if (!PARSE_SWITCH(MaxOutputFileSize))
			{
				// Failing to parse a provided MaxOutputFileSize is an error.
				UE_LOGF(LogBuildPatchTool, Error, "MaxOutputFileSize must be a valid uint64");
				return false;
			}

			if (MaxOutputFileSize < 10000000)
			{
				// MaxOutputFileSize must be at least 10MB, otherwise fail.
				UE_LOGF(LogBuildPatchTool, Error, "MaxOutputFileSize must be at least 10000000 (10MB)");
				return false;
			}
		}
		else
		{
			// If not provided we don't limit the size, which is the equivalent of limiting to max uint64.
			MaxOutputFileSize = TNumericLimits<uint64>::Max();
		}

		// Process the tagsets that we parsed.
		if (TagSets.Num() > 0)
		{
			for (const FString& TagSet : TagSets)
			{
				TArray<FString> Tags;
				const bool bCullEmpty = false;
				TagSet.ParseIntoArray(Tags, TEXT(","), bCullEmpty);
				for (FString& Tag : Tags)
				{
					Tag.TrimStartAndEndInline();
				}
				// If we ended up with an empty array, the intention would have been to pass a tagset that contains just an empty string, so make that fixup.
				if (Tags.Num() == 0)
				{
					Tags.Add(TEXT(""));
				}
				TagSetArray.Emplace(Tags);
			}
		}

		bIgnoreManifestFileInvalidTags = ParseOption(TEXT("IgnoreManifestFileInvalidTags"), Switches);
		bIgnorePrevManifestFileInvalidTags = ParseOption(TEXT("IgnorePrevManifestFileInvalidTags"), Switches);

		// Parse all the secret key and ID pairs.
		FString ParseErrorMessage;
		if (!ParsePairs(TEXT("EncryptionSecretId="), TEXT("EncryptionSecretKey="), Switches, EncryptionSecrets, ParseErrorMessage) && !ParseErrorMessage.IsEmpty())
		{
			UE_LOGF(LogBuildPatchTool, Error, "%ls", *ParseErrorMessage);
			return false;
		}

		return true;
#undef PARSE_SWITCHES
#undef PARSE_SWITCH
#undef HAS_DUP_SWITCH
#undef HAS_SWITCH
	}

	void LogConfig() const
	{
		UE_LOGF(LogBuildPatchTool, Log, "-----Configuration for PackageChunksMode------");
		UE_LOGF(LogBuildPatchTool, Log, "   ManifestFile: %ls", *ManifestFile);
		UE_LOGF(LogBuildPatchTool, Log, "   PrevManifestFile: %ls", *PrevManifestFile);
		UE_LOGF(LogBuildPatchTool, Log, "   OutputFile: %ls", *OutputFile);
		UE_LOGF(LogBuildPatchTool, Log, "   ResultDataFile: %ls", *ResultDataFile);
		UE_LOGF(LogBuildPatchTool, Log, "   CloudDir: %ls", *CloudDir);
		UE_LOGF(LogBuildPatchTool, Log, "   MaxOutputFileSize: %llu", MaxOutputFileSize);
		for (const TSet<FString>& TagSet : TagSetArray)
		{
			FString TagSetString = FString::Join(TagSet.Array(), TEXT(","));
			UE_LOGF(LogBuildPatchTool, Log, "   With Tag set: %ls", *TagSetString);
		}
		FString PrevTagSetString = FString::Join(PrevTagSet.Array(), TEXT(","));
		UE_LOGF(LogBuildPatchTool, Log, "   Previous Tag set: %ls", *PrevTagSetString);
		if (bIgnoreManifestFileInvalidTags)
		{
			UE_LOGF(LogBuildPatchTool, Log, "   IgnoreManifestFileInvalidTags: Enabled");
		}
		if (bIgnorePrevManifestFileInvalidTags)
		{
			UE_LOGF(LogBuildPatchTool, Log, "   IgnorePrevManifestFileInvalidTags: Enabled");
		}
		for (const TPair<FGuid, TArray<uint8>>& EncryptionSecret : EncryptionSecrets)
		{
			UE_LOGF(LogBuildPatchTool, Log, "   AvailableSecretId: %ls", *EncryptionSecret.Key.ToString());
		}
	}

private:
	static TCHAR const* const MODE_NAME;
	IBuildPatchServicesModule& BpsInterface;
	const TCHAR* CommandLine;
	bool bHelp;
	FString FeatureLevel;
	FString ManifestFile;
	FString PrevManifestFile;
	FString OutputFile;
	FString ResultDataFile;
	FString DeltaFilenameTrailer;
	FString CloudDir;
	uint64 MaxOutputFileSize;
	TArray<TSet<FString>> TagSetArray;
	TSet<FString> PrevTagSet;
	bool bIgnoreManifestFileInvalidTags;
	bool bIgnorePrevManifestFileInvalidTags;
	TMap<FGuid, TArray<uint8>> EncryptionSecrets;
};

IMPLEMENT_BPT_MODE(PackageChunks, FPackageChunksToolMode);
