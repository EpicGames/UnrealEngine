// Copyright Epic Games, Inc. All Rights Reserved.
#include "ToolModes/ToolModesHelp.h"

namespace BuildPatchTool
{
		template<typename T>
		static constexpr int32 GetNumOfLines()
		{
			return sizeof(T::Text) / sizeof(T::Text[0]);
		}

#define DEFINE_NUM_OF_LINES(_x_) const int32 _x_::NumLines = GetNumOfLines<_x_>();

	const TCHAR* FBptBaseToolModeHelp::Text[] {
		TEXT("BUILDPATCHTOOL v" BUILDPATCHTOOL_VERSION_STRING),
		TEXT(""),
		TEXT("-help can be added with any mode selection to get extended usage information."),
		TEXT(""),
		TEXT("Available modes are:"),
		TEXT("  -mode=ChunkBuildDirectory    Generates patch data for a new binary."), 
		TEXT("  -mode=ChunkDeltaOptimise     Produces a more optimal chunked patch between two specific binaries."),
		TEXT("  -mode=ExtractMetaData        Takes a manifest and produces a portable output with information about the contents in human or json form."),
		TEXT("  -mode=InstallManifest        Can rebuild the files given a manifest and clouddir source."),
		TEXT("  -mode=DiffManifests          Diffs the manifests for two binaries and outputs statistics for applying the update."),
		TEXT("  -mode=Compactify             Cleans up unneeded patch data from a given cloud directory with redundant data."),
		TEXT("  -mode=Enumeration            Outputs the paths to referenced patch data given a single manifest."),
		TEXT("  -mode=MergeManifests         Combines two manifest files to create a new one, primarily used to create hotfixes."),
		TEXT("  -mode=PackageChunks          Packages data required for an installation into larger files which can be used as local sources for build patch installers."),
		TEXT("  -mode=VerifyChunks           Allows you to verify the integrity of patch data. It will load chunk or chunkdb files to verify they are not corrupt.")
	#if !UE_BUILD_SHIPPING && WITH_DEV_AUTOMATION_TESTS
		TEXT("  -mode=AutomationTests        Runs automation testing."),
	#endif // !UE_BUILD_SHIPPING && WITH_DEV_AUTOMATION_TESTS
	};

	DEFINE_NUM_OF_LINES(FBptBaseToolModeHelp);

	const TCHAR* FDiffManifestToolModeHelp::Text[]{
		TEXT("DIFF MANIFEST MODE"),
		TEXT("This tool mode reports the changes between two existing manifest files."),
		TEXT(""),
		TEXT("Required arguments:"),
		TEXT("  -mode=DiffManifests      Must be specified to launch the tool in diff manifests mode."),
		TEXT("  -ManifestA=\"\"            Specifies in quotes the file path to the base manifest."),
		TEXT("  -ManifestB=\"\"            Specifies in quotes the file path to the update manifest."),
		TEXT(""),
		TEXT("Optional arguments:"),
		TEXT("  -InstallTagsA=\"\"         Specifies in quotes a comma separated list of install tags used on ManifestA. You should include empty string if you want to count untagged files."),
		TEXT("                           Leaving the parameter out will use all files."),
		TEXT("                           -InstallTagsA=\"\" will be untagged files only."),
		TEXT("                           -InstallTagsA=\",tag\" will be untagged files plus files tagged with 'tag'."),
		TEXT("                           -InstallTagsA=\"tag\" will be files tagged with 'tag' only."),
		TEXT("  -InstallTagsB=\"\"         Specifies in quotes a comma separated list of install tags used on ManifestB. Same rules apply as InstallTagsA."),
		TEXT("  -CompareTagSet=\"\"        Specifies in quotes a comma separated list of install tags used to calculate differential statistics betweeen the manifests. Multiple lists are allowed. Same rules apply as InstallTagsA."),
		TEXT("  -OutputFile=\"\"           Specifies in quotes the file path where the diff will be exported as a JSON object."),
		TEXT("  -PatchDescriptorPath=\"\"  If specified, will write out metadata files specifying what parts of each patched file changed."),
		TEXT("  -OnlyPatchDescriptors    If specified with PatchDescriptorPath, DiffManifests will only emit the files and not emit diff info."),
		TEXT("  -RequireOptimizedDelta   If specified, fail if an optimized delta file doesn't exist.")
		TEXT(""),
	};

	DEFINE_NUM_OF_LINES(FDiffManifestToolModeHelp);

	const TCHAR* FChunkDirectoryToolModeHelp::Text[]{
		TEXT("GENERATE CHUNKS FROM DIRECTORY MODE"),
		TEXT("This tool supports generating chunk based patches. The resulting binary can optionally be uploaded to Epic services."),
		TEXT(""),
		TEXT("Required arguments:"),
		TEXT("  -mode=ChunkBuildDirectory    Must be specified to launch the tool in patch generation mode."),
		TEXT("  -FeatureLevel=Latest         Specifies the client feature level to output data for. See BuildPatchServices::EFeatureLevel for possible values."),
		TEXT("  -BuildRoot=\"\"                Specifies in quotes the directory containing the binary image to be read."),
		TEXT("  -CloudDir=\"\"                 Specifies in quotes the cloud directory where existing data will be recognized from, and new data added to."),
		TEXT("  -ArtifactId=\"\"               Specifies in quotes, the id of the artifact."),
		TEXT("  -BuildVersion=\"\"             Specifies in quotes, the version string for the binary image."),
		TEXT("  -AppLaunch=\"\"                Specifies in quotes, the path to the app executable, must be relative to, and inside of BuildRoot."),
		TEXT("  -AppArgs=\"\"                  Specifies in quotes, the commandline to send to the app on launch."),
		TEXT(""),
		TEXT("Optional arguments:"),
		TEXT("  -AppID=123456                Specifies without quotes, the ID number for the app. This will default to 0 if not provided."),
		TEXT("  -FileList=\"\"                 Specifies in quotes, the path to a text file containing BuildRoot relative files to be included in the binary."),
		TEXT("  -FileIgnoreList=\"\"           Specifies in quotes, the path to a text file containing BuildRoot relative files, separated by \\r\\n line endings, to not be included in the binary."),
		TEXT("  -FileAttributeList=\"\"        Specifies in quotes, the path to a text file containing BuildRoot relative files and corresponding special attributes (e.g. executable, readonly, compressed)."),
		TEXT("                               Your custom FileAttributeList file must be a text file with one or more newline-separated entries."),
		TEXT("                               Each line must contain a quoted (BuildRoot-relative) filepath, followed by one or more space-delimited attributes to be applied to the referenced file."),
		TEXT("  -FileSortOrder=OrderValue    Specifies the order in which files should be scanned. The default and recommended is AlphabeticalTagThenFilename, however AlphabeticalFilename is also available for legacy behaviour if necessary."),
		TEXT("                               AlphabeticalTagThenFilename will scan, and thus install, files in the alphabetical order of their assigned tags (see FileAttributeList) then by filename per tag."),
		TEXT("                               AlphabeticalFilename will scan, and thus install, files in the alphabetical order of their filename only."),
		TEXT("  -PrereqIds=\"\"                Specifies in quotes, a comma-separated list of identifiers that the prerequisites satisfy."),
		TEXT("                               At install time, a machine which already has installed prerequisites with all of these ids will skip prerequisite installation."),
		TEXT("  -PrereqName=\"\"               Specifies in quotes, the display name for the prerequisites installer."),
		TEXT("  -PrereqPath=\"\"               Specifies in quotes, the prerequisites installer to launch on successful product install, relative to the build root."),
		TEXT("  -PrereqArgs=\"\"               Specifies in quotes, the commandline to send to prerequisites installer on launch."),
		TEXT("                               This value supports string replacements for \"$[RootDirectory]\" and \"$[LogDirectory]\"."),
		TEXT("                               RootDirectory will be replaced with the root path of the installed game before executing."),
		TEXT("                               LogDirectory is the path to the program's log output directory so your prereq could create logs there."),
		TEXT("                               The replacements will include trailing /."),
		TEXT("                               \"$[Quote]\" can also be used to get a quote character, this is important because the BPT commandline already uses quotes for token parsing."),
		TEXT("  -UninstallActionPath=\"\"      Specifies in quotes, the custom uninstall action executable to launch right before product uninstall, relative to the build root."),
		TEXT("  -UninstallActionArgs=\"\"      Specifies in quotes, the commandline to send to the custom uninstall action on launch."),
		TEXT("  -DataAgeThreshold=12.5       Specified the maximum age (in days) of existing manifest files whose referenced patch data can be reused in the generated manifest."),
		TEXT("  -Custom=\"field=value\"        Adds a custom string field to the binary's manifest."),
		TEXT("  -CustomInt=\"field=number\"    Adds a custom int64 field to the binary's manifest."),
		TEXT("  -CustomFloat=\"field=number\"  Adds a custom double field to the binary's manifest."),
		TEXT("  -OutputFilename=\"\"           Specifies in quotes an override for the output manifest filename. Extension of .manifest will be added if not present."),
		TEXT("  -ChunkWindowSize=1000000     Specifies in bytes, the data window size that should be used when saving new chunks. Default is 1048576 (1MiB)."),
		TEXT("  -IgnoreOtherWindowSizes      If provided, the generation code will only accept chunk matches that are the same as ChunkWindowSize."),
		TEXT("  -MetadataOutput=\"\"           Specifies in quotes, if provided, the path on disk to save the extracted information from resulted manifest. If path ends with .txt data is transformed into human-readable format otherwise json."),
		TEXT(""),
		TEXT("Note: If -DataAgeThreshold is not supplied on the command-line, then all existing data is eligible for reuse in the generated manifest."),
		TEXT("Note: If -OutputFilename is not supplied on the command-line, the default of ArtifactIdBuildVersion.manifest will be used for local operations."),
		TEXT("Note:    -OutputFilename must be a clean filename with no path."),
		TEXT("Note: If -ChunkWindowSize provided, it is clamped max 10485760 (10MiB) to min 32000 (32KB)."),
		TEXT(""),
	};

	DEFINE_NUM_OF_LINES(FChunkDirectoryToolModeHelp);

	const TCHAR* FAutomationToolModeHelp::Text[]{
		TEXT("AUTOMATION TEST MODE"),
		TEXT("This tool mode runs automation tests."),
		TEXT(""),
		TEXT("Required arguments:"),
		TEXT("  -mode=AutomationTests  Must be specified to launch the tool in automation tests mode."),
		TEXT(""),
		TEXT("Optional arguments:"),
		TEXT("  -TestList=\"\"           Specifies in quotes, the list of tests to run. The list is + delimited."),
		TEXT(""),
		TEXT("Note: If -TestList is not specified, then all BuildPatchTool and BuildPatchServices tests are ran."),
		TEXT(""),
	};

	DEFINE_NUM_OF_LINES(FAutomationToolModeHelp);

	const TCHAR* FChunkDeltaOptimiseModeHelp::Text[]{
		TEXT("CHUNK DELTA OPTIMISE MODE"),
		TEXT("This tool supports optimising chunk based patches to reduce the number of chunks required to download when patching between specific versions."),
		TEXT(""),
		TEXT("Required arguments:"),
		TEXT("  -mode=ChunkDeltaOptimise  Must be specified to launch the tool in chunk delta optimise mode."),
		TEXT("  -ManifestA=\"\"             Specifies in quotes the file path to the base manifest."),
		TEXT("  -ManifestB=\"\"             Specifies in quotes the file path to the update manifest."),
		TEXT(""),
		TEXT("Optional arguments:"),
		TEXT("  -CloudDir=\"\"                     Specifies in quotes the cloud directory where new data will be saved. If not provided, location of ManifestB will be used as output."),
		TEXT("  -DownloadCloudDirectory=\"\"       Specifies in quotes the cloud directory where existing data can be downloaded. If not provided, location of ManifestB will be used as source."),
		TEXT("                                   Multiple DownloadCloudDirectory arguments can be provided."),
		TEXT("  -ScanWindowSize=1000000          Specifies in bytes, the scan window to use. Range accepted is 128KiB >= n >= 128b, defaults to 8191 (Closest prime to 8KiB)."),
		TEXT("  -OutputChunkSize=1000000         Specifies in bytes, the chunk size to save out unknown data with. Range accepted is 10MiB >= n >= 1MB, defaults to 1048576 (1MiB)."),
		TEXT("  -DiffAbortThreshold=             Specifies an upper limit for original diffs to try to enhance, expressed as an absolute value or a percentage."),
		TEXT("                                   This allows short circuiting lengthy optimisation attempts on large diffs which may not benefit."),
		TEXT("                                   * Absolute values can be specified using data units such as : KiB, Kb, MiB, Mb, GiB, and Gb."),
		TEXT("                                   * Values without units are interpreted as bytes."),
		TEXT("                                   * Percentages can be explicitly specified using units such as: Pc or Pct."),
		TEXT("                                   The accepted range is `n >= 1GB` or the equivalent percentage of `BuildVersionB` size. Defaults to never abort."),
		TEXT(""),
	};

	DEFINE_NUM_OF_LINES(FChunkDeltaOptimiseModeHelp);

	const TCHAR* FCompactifyToolModeHelp::Text[]{
		TEXT("COMPACTIFY MODE"),
		TEXT("This tool supports the removal of redundant patch data from a cloud directory."),
		TEXT(""),
		TEXT("Required arguments:"),
		TEXT("  -mode=Compactify           Must be specified to launch the tool in compactify mode."),
		TEXT("  -CloudDir=\"\"               Specifies in quotes the cloud directory where manifest files and chunks to be compactified can be found."),
		TEXT("  -DataAgeThreshold=2        The minimum age in days of chunk files that will be deleted. Any unreferenced chunks older than this will be deleted."),
		TEXT(""),
		TEXT("Optional arguments:"),
		TEXT("  -preview                   Log all the actions it will take to update internal structures, but don't actually execute them."),
		TEXT("  -DeletedChunkLogFile=\"\"    Log the list of paths of deleted chunk files to this specified filename. All paths are relative to CloudDir."),
		TEXT(""),
		TEXT("Note: If -DataAgeThreshold is specified as 0, then all unreferenced existing data is eligible for deletion by the compactify process."),
		TEXT(""),
	};

	DEFINE_NUM_OF_LINES(FCompactifyToolModeHelp);

	const TCHAR* FEnumerationToolModeHelp::Text[]{
		TEXT("ENUMERATION MODE"),
		TEXT("This tool supports enumerating patch data referenced by a manifest or chunkdb."),
		TEXT(""),
		TEXT("Required arguments:"),
		TEXT("  -mode=Enumeration    Must be specified to launch the tool in enumeration mode."),
		TEXT("  -InputFile=\"\"        Specifies in quotes the file path to the manifest or chunkdb to enumerate from."),
		TEXT("  -OutputFile=\"\"       Specifies in quotes the file path to a file where the list will be saved out."),
		TEXT(""),
		TEXT("Optional arguments:"),
		TEXT("  -includesizes        When specified, the size of each file in bytes will also be output (see notes)."),
		TEXT(""),
		TEXT("Note: If InputFile is a manifest, the output file format will be text file with one line per file, each containing cloud relative path."),
		TEXT("      e.g. path/to/file"),
		TEXT("      If InputFile is a chunk package, the output file format will be text file with one line per chunk, each containing tab separated hex chunk"),
		TEXT("      GUID, hex chunk rolling hash, and hex chunk SHA1."),
		TEXT("      e.g. 2CC26D05B64363780D5CF292E6B570A3\\t078070129133079067060057\\t527490FCA1DA6FAAB0E6F6E369E372FA693CCFBB"),
		TEXT("      Line endings are \\r\\n."),
		TEXT("Note: If includesizes is specified, each line of the output text file will end with a tab then the number of bytes of the file."),
		TEXT(""),
	};

	DEFINE_NUM_OF_LINES(FEnumerationToolModeHelp);

	const TCHAR* FExtractMetadataToolModeHelp::Text[]{
		TEXT("EXTRACT METADATA FROM MANIFEST MODE"),
		TEXT("This tool outputs details about the file provided if it is a BuildPatchTool based data format."),
		TEXT(""),
		TEXT("Required arguments:"),
		TEXT("  -mode=ExtractMetadata    Must be specified to launch the tool in extract metadata mode."),
		TEXT("  -InputFile=\"\"            Specifies in quotes, the path to the input file on disk. There can be multiple InputFile but each must be followed by OutputFile."),
		TEXT("  -OutputFile=\"\"           Specifies in quotes, the path on disk to save the output."),
		TEXT(""),
		TEXT("Optional arguments:"),
		TEXT("  -OutputFormat=           Specifies the format for the output file, currently supports human or json, default json."),
		TEXT(""),
	};

	DEFINE_NUM_OF_LINES(FExtractMetadataToolModeHelp);

	const TCHAR* FInstallManifestToolModeHelp::Text[]{
		TEXT("INSTALL FILES FROM MANIFEST MODE"),
		TEXT("This tool supports installing files from a manifest."),
		TEXT(""),
		TEXT("Required arguments:"),
		TEXT("  -mode=InstallManifest    Must be specified to launch the tool in install manifest mode."),
		TEXT("  -Manifest=\"\"             Manifest of the binary to install."),
		TEXT("  -OutputDir=\"\"            Output directory for the installation."),
		TEXT(""),
		TEXT("Optional arguments:"),
		TEXT("  -PrevManifest=\"\"         Manifest of the binary already installed to OutputDir."),
		TEXT("  -InstallTags=\"\"          Specifies a comma-separated list of install tags to use for selectively installing files. If left empty or missing, all files will be installed. Mutually exclusive with FileList."),
		TEXT("  -FileList=\"\"             Specifies a comma separated list of files in the manifest to filter the installation for. Mutually exclusive with InstallTags."),
		TEXT("  -CloudDirs=\"\"            Specifies in quotes, a comma separated list of cloud directory roots where chunks can be sourced."),
		TEXT("  -RejectSymlinks          If specified, no symlinks are allowed to be used in manifest file."),
		TEXT(""),
		TEXT("Note: If CloudDirs param is not used, the path to the manifest will be used."),
		TEXT(""),
	};

	DEFINE_NUM_OF_LINES(FInstallManifestToolModeHelp);

	const TCHAR* FMergeManifestToolModeHelp::Text[]{
		TEXT("MERGE MANIFEST MODE"),
		TEXT("This tool supports generating a hotfix manifest from two existing manifest files."),
		TEXT(""),
		TEXT("Required arguments:"),
		TEXT("  -mode=MergeManifests    Must be specified to launch the tool in merge manifests mode."),
		TEXT("  -ManifestA=\"\"           Specifies in quotes the file path to the base manifest."),
		TEXT("  -ManifestB=\"\"           Specifies in quotes the file path to the update manifest."),
		TEXT("  -ManifestC=\"\"           Specifies in quotes the file path to the output manifest."),
		TEXT("  -BuildVersion=\"\"        Specifies in quotes the new version string for the binary being produced."),
		TEXT(""),
		TEXT("Optional arguments:"),
		TEXT("  -MergeFileList=\"\"       Specifies in quotes, the path to a text file containing complete list of desired build root relative files followed by \\t character, followed by A or B to select the manifest to pull from. These should be separated by \\r\\n line endings."),
		TEXT(""),
		TEXT("Note: If -MergeFileList is not specified, then union of all files will be selected, preferring ManifestB's version."),
		TEXT("Note: With the exception of the new version string for the binary, all meta will be copied from only ManifestB."),
		TEXT(""),
	};

	DEFINE_NUM_OF_LINES(FMergeManifestToolModeHelp);

	const TCHAR* FPackageChunksToolModeHelp::Text[]{
		TEXT("PACKAGE CHUNKS MODE"),
		TEXT("This tool mode supports packaging data required for an installation into larger files which can be used as local sources for build patch installers."),
		TEXT(""),
		TEXT("Required arguments:"),
		TEXT("  -mode=PackageChunks    Must be specified to launch the tool in package chunks mode."),
		TEXT("  -FeatureLevel=Latest   Specifies the client feature level to output data for. See BuildPatchServices::EFeatureLevel for possible values."),
		TEXT("  -ManifestFile=\"\"       Specifies in quotes the file path to the manifest to enumerate chunks from."),
		TEXT("  -OutputFile=\"\"         Specifies in quotes the file path the output package. Extension of .chunkdb will be added if not present."),
		TEXT(""),
		TEXT("Optional arguments:"),
		TEXT("  -PrevManifestFile=\"\"   Specifies in quotes the file path to a manifest for a previous binary, this will be used to filter out chunks, such that the"),
		TEXT("                         produced chunkdb files will only contain chunks required to patch from this binary to the one described by ManifestFile."),
		TEXT("  -CloudDir=\"\"           Specifies in quotes the cloud directory where chunks to be packaged can be found."),
		TEXT("  -MaxOutputFileSize=    When specified, the size of each output file (in bytes) will be limited to a maximum of the provided value."),
		TEXT("  -ResultDataFile=\"\"     Specifies in quotes the file path where the results will be exported as a JSON object."),
		TEXT("  -TagSets=\",t1,t2\"      Specifies in quotes a comma separated tagset for filtering of data saved. Multiple sets can also be provided to split the chunkdb files by tagsets."),
		TEXT("                         Untagged files will be referenced with an empty tag, which you can specify using an extra comma."),
		TEXT("  -PrevTagSet=\",tA,tB\"   Specifies in quotes a comma separated tagset for filtering of input data usable from PrevManifestFile. This will increase the amount of chunks"),
		TEXT("                         saved out by reducing the number of files from the input manifest that are assumed usable. Only one PrevTagSet should be provided."),
		TEXT("                         Untagged files will be referenced with an empty tag, which you can specify using an extra comma."),
		TEXT(""),
		TEXT("Note: If CloudDir is not specified, the manifest file location will be used as the cloud directory."),
		TEXT("Note: If an optimised delta was available, the file extension .delta.chunkdb will be used."),
		TEXT("Note: MaxOutputFileSize must be at least 10000000 (10MB)."),
		TEXT("Note: MaxOutputFileSize is recommended to be as large as possible. The minimum individual chunkdb filesize is equal to one chunk plus chunkdb"),
		TEXT("      header, and thus will not result in efficient behavior."),
		TEXT("Note: If MaxOutputFileSize is not specified, the one output file will be produced containing all required data."),
		TEXT("Note: If MaxOutputFileSize is specified, the output files will be generated as Name.part01.chunkdb, Name.part02.chunkdb etc. The part number will"),
		TEXT("      have the number of digits required for highest numbered part."),
		TEXT("Note: If MaxOutputFileSize is specified, then each part can be equal to or less than the specified size, depending on the size of the last chunk"),
		TEXT("      that fits."),
		TEXT("Note: When providing multiple -TagSets= arguments, all data from the first -TagSets= arg will be saved first, followed by any extra data needed for the second -TagSets= arg, and so on in separated chunkdb files."),
		TEXT("      Note that this means the chunkdb files produced for the second -TagSets= arg and later will not contain some required data for that tagset if the data already got saved out as part of a previous tagset."),
		TEXT("      The chunkdb files are thus additive with no dupes."),
		TEXT("      If it is desired that each tagset's chunkdb files contain the duplicate data, then PackageChunks should be executed once per -TagSets= arg rather than once will all -TagSets= args."),
		TEXT("      An empty tag must be included in one of the -TagSets= args to include untagged file data in that tagset, e.g. -TagSets=\" , t1\"."),
		TEXT("      Adding no -TagSets= args will include all data."),
		TEXT("Note: ManifestFile and PrevManifestFile should point to different files."),
		TEXT(""),
	};

	DEFINE_NUM_OF_LINES(FPackageChunksToolModeHelp);

	const TCHAR* FVerifyChunksToolModeHelp::Text[]{
		TEXT("VERIFY CHUNKS MODE"),
		TEXT("This tool mode allows you to verify the integrity of patch data. It will load chunk or chunkdb files to verify they are not corrupt."),
		TEXT(""),
		TEXT("Required arguments:"),
		TEXT("  -mode=VerifyChunks   Must be specified to launch the tool in verify chunks mode."),
		TEXT("  -SearchPath=\"\"       Specifies in quotes the directory path which contains data to verify."),
		TEXT(""),
		TEXT("Optional arguments:"),
		TEXT("  -OutputFile=\"\"       When specified, full file path for each bad data will be saved to this file as \\r\\n separated list."),
		TEXT(""),
		TEXT("Note: All checks are logged, normal log for good data, error log for any bad data found."),
		TEXT(""),
	};

	DEFINE_NUM_OF_LINES(FVerifyChunksToolModeHelp);

#undef OFFLINE_MODES_TEXT
}
