// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BuildPatchToolVersion.h"
#include "HAL/Platform.h"

namespace BuildPatchTool
{
	struct FBptBaseToolModeHelp
	{
		static const TCHAR* Text[];
		static const int32 NumLines;
	};

	struct FDiffManifestToolModeHelp
	{
		static const TCHAR* Text[];
		static const int32 NumLines;
	};

	struct FChunkDirectoryToolModeHelp
	{
		static const TCHAR* Text[];
		static const int32 NumLines;
	};

	struct FAutomationToolModeHelp
	{
		static const TCHAR* Text[];
		static const int32 NumLines;
	};

	struct FChunkDeltaOptimiseModeHelp
	{
		static const TCHAR* Text[];
		static const int32 NumLines;
	};

	struct FCompactifyToolModeHelp
	{
		static const TCHAR* Text[];
		static const int32 NumLines;
	};

	struct FEnumerationToolModeHelp
	{
		static const TCHAR* Text[];
		static const int32 NumLines;
	};

	struct FExtractMetadataToolModeHelp
	{
		static const TCHAR* Text[];
		static const int32 NumLines;
	};

	struct FInstallManifestToolModeHelp
	{
		static const TCHAR* Text[];
		static const int32 NumLines;
	};

	struct FMergeManifestToolModeHelp
	{
		static const TCHAR* Text[];
		static const int32 NumLines;
	};

	struct FPackageChunksToolModeHelp
	{
		static const TCHAR* Text[];
		static const int32 NumLines;
	};

	struct FVerifyChunksToolModeHelp
	{
		static const TCHAR* Text[];
		static const int32 NumLines;
	};
}
