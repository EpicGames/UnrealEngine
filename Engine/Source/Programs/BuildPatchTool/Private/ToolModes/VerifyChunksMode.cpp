// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolModes/VerifyChunksMode.h"

#include "Algo/Find.h"
#include "BuildPatchTool.h"
#include "Interfaces/IBuildPatchServicesModule.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "ToolModes/ToolModesHelp.h"

using namespace BuildPatchTool;

class FVerifyChunksToolMode : public IToolMode
{
public:
	FVerifyChunksToolMode(IBuildPatchServicesModule& InBpsInterface, const TCHAR* InCommandLine)
		: BpsInterface(InBpsInterface)
		, CommandLine(InCommandLine)
	{}

	virtual ~FVerifyChunksToolMode()
	{}

	virtual EReturnCode Execute() override
	{
		// Parse commandline.
		if (ProcessCommandline() == false)
		{
			return EReturnCode::ArgumentProcessingError;
		}

		// Print help if requested.
		if (bHelp)
		{
			PrintHelp<FVerifyChunksToolModeHelp>();
			return EReturnCode::OK;
		}

		LogConfig();

		// Register secrets.
		for (const TPair<FGuid, TArray<uint8>>& IdKeyPair : EncryptionSecrets)
		{
			BpsInterface.RegisterEncryptionSecret(IdKeyPair.Key, IdKeyPair.Value);
		}

		// Run the enumeration routine.
		bool bSuccess = BpsInterface.VerifyChunkData(SearchPath, OutputFile);
		return bSuccess ? EReturnCode::OK : EReturnCode::ToolFailure;
	}

private:

	bool ProcessCommandline()
	{
#define HAS_SWITCH(Switch) (Algo::FindByPredicate(Switches, [](const FString& Elem){ return Elem.StartsWith(TEXT(#Switch "="));}) != nullptr)
#define PARSE_SWITCH(Switch) ParseSwitch(TEXT(#Switch "="), Switch, Switches)
		TArray<FString> Tokens, Switches;
		FCommandLine::Parse(CommandLine, Tokens, Switches);

		bHelp = ParseOption(TEXT("help"), Switches);
		if (bHelp)
		{
			return true;
		}

		// Get all required parameters.
		if (!PARSE_SWITCH(SearchPath))
		{
			UE_LOGF(LogBuildPatchTool, Error, "SearchPath is a required parameter");
			return false;
		}

		if (SearchPath.IsEmpty())
		{
			UE_LOGF(LogBuildPatchTool, Error, "SearchPath argument can't be empty");
			return false;
		}

		NormalizeUriPath(SearchPath);

		// Get optional parameters.
		PARSE_SWITCH(OutputFile);
		NormalizeUriFile(OutputFile);

		// Parse all the secret key and ID pairs.
		FString ParseErrorMessage;
		if (!ParsePairs(TEXT("EncryptionSecretId="), TEXT("EncryptionSecretKey="), Switches, EncryptionSecrets, ParseErrorMessage) && !ParseErrorMessage.IsEmpty())
		{
			UE_LOGF(LogBuildPatchTool, Error, "%ls", *ParseErrorMessage);
			return false;
		}

		return true;
#undef PARSE_SWITCH
#undef HAS_SWITCH
	}

	void LogConfig() const
	{
		UE_LOGF(LogBuildPatchTool, Log, "-----Configuration for VerifyChunksMode------");
		UE_LOGF(LogBuildPatchTool, Log, "   SearchPath: %ls", *SearchPath);
		UE_LOGF(LogBuildPatchTool, Log, "   OutputFile: %ls", *OutputFile);
		for (const TPair<FGuid, TArray<uint8>>& EncryptionSecret : EncryptionSecrets)
		{
			UE_LOGF(LogBuildPatchTool, Log, "   AvailableSecretId: %ls", *EncryptionSecret.Key.ToString());
		}
	}

private:
	static TCHAR const* const MODE_NAME;
	IBuildPatchServicesModule& BpsInterface;
	const TCHAR* CommandLine;
	bool bHelp;
	FString SearchPath;
	FString OutputFile;
	TMap<FGuid, TArray<uint8>> EncryptionSecrets;
};

IMPLEMENT_BPT_MODE(VerifyChunks, FVerifyChunksToolMode);
