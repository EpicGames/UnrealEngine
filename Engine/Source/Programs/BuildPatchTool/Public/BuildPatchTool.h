// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

DECLARE_LOG_CATEGORY_EXTERN(LogBuildPatchTool, Log, All);

namespace BuildPatchTool
{
	enum class EReturnCode : int32
	{
		OK = 0,
		UnknownError,
		ArgumentProcessingError,
		UnknownToolMode,
		FileNotFound,
		ToolFailure,
		OnlineMcpError,
		DeprecatedVersion,
		DirectoryNotFound,
		MissingCredentials,
		InvalidCredentials,
		InvalidData,

		// !! Always after the last code, signifies the last return code plus 1 to allow the following Last alias.
		LastPlusOne,
		// An alias for the actual last return code.
		Last = (LastPlusOne - 1),

		Crash = 255
	};
}

namespace CommandLineHelpers
{
	bool ParseSwitch(const TCHAR* InSwitch, FString& Value, const TArray<FString>& Switches);
}

inline const TCHAR* LexToString(BuildPatchTool::EReturnCode ReturnCode)
{
	static_assert(BuildPatchTool::EReturnCode::Last == BuildPatchTool::EReturnCode::InvalidData, "Please add support for the extra values below.");
#define CASE_ENUM_TO_STR(Value) case BuildPatchTool::EReturnCode::Value: return TEXT(#Value)
	switch (ReturnCode)
	{
		CASE_ENUM_TO_STR(OK);
		CASE_ENUM_TO_STR(UnknownError);
		CASE_ENUM_TO_STR(ArgumentProcessingError);
		CASE_ENUM_TO_STR(UnknownToolMode);
		CASE_ENUM_TO_STR(FileNotFound);
		CASE_ENUM_TO_STR(ToolFailure);
		CASE_ENUM_TO_STR(OnlineMcpError);
		CASE_ENUM_TO_STR(DeprecatedVersion);
		CASE_ENUM_TO_STR(DirectoryNotFound);
		CASE_ENUM_TO_STR(MissingCredentials);
		CASE_ENUM_TO_STR(InvalidCredentials);
		CASE_ENUM_TO_STR(InvalidData);
		CASE_ENUM_TO_STR(Crash);
	default: return TEXT("InvalidOrMax");
	}
#undef CASE_ENUM_TO_STR
}
