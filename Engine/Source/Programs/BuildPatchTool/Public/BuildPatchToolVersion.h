// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Change the following three defines to set the version of BuildPatchTool
#define BUILDPATCHTOOL_MAJOR_VERSION	1
#define BUILDPATCHTOOL_MINOR_VERSION	9
#define BUILDPATCHTOOL_PATCH_VERSION	0

//
// DO NOT CHANGE THE BELOW DEFINITIONS
//

// Macros for encoding strings
#define VERSION_TEXT(x) TEXT(x)
#define VERSION_STRINGIFY_2(x) VERSION_TEXT(#x)
#define VERSION_STRINGIFY(x) VERSION_STRINGIFY_2(x)

// Macro to build up full major.minor.patch version string for BuildPatchTool
#define BUILDPATCHTOOL_VERSION_STRING \
	VERSION_STRINGIFY(BUILDPATCHTOOL_MAJOR_VERSION) \
	VERSION_TEXT(".") \
	VERSION_STRINGIFY(BUILDPATCHTOOL_MINOR_VERSION) \
	VERSION_TEXT(".") \
	VERSION_STRINGIFY(BUILDPATCHTOOL_PATCH_VERSION)
