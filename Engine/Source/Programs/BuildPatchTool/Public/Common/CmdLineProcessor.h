// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "Templates/Function.h"
#include "Misc/Paths.h"
#include "Common/DiffAbortThresholdArgument.h"

#define BPT_DECLARE_COMMANDLINE(TConfig) \
bool ProcessCommandLine(); \
static TCHAR const * const MODE_NAME; \
TConfig Config;

#define BPT_COMMANDLINE_START(TClass) \
bool TClass::ProcessCommandLine() \
{ \
	bool bParamExists = true; \
	bool bSuccess = true; \
	TArray<FString> Tokens, Switches; \
	FCommandLine::Parse(CommandLine, Tokens, Switches); \
	bHelp = ParseOption(TEXT("help"), Switches); \
	if (bHelp) \
	{ \
		return true; \
	}

#define BPT_ARG_REQUIRED_SWITCH(Switch) \
	bParamExists = ParseSwitch(TEXT(#Switch "="), Config.Switch, Switches); \
	if (!bParamExists) \
	{ \
		UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("{0} is required for {1} mode."), { TEXT(#Switch), MODE_NAME })); \
		bSuccess = false; \
	}

#define BPT_ARG_REQUIRED_SWITCH_PAIRS(Key, Val, Map) \
	{ \
		FString ParseErrorMessage; \
		bParamExists = ParsePairs(TEXT(#Key "="), TEXT(#Val "="), Switches, Config.Map, ParseErrorMessage); \
		if (!ParseErrorMessage.IsEmpty()) \
		{ \
			UE_LOGF(LogBuildPatchTool, Error, "%ls", *ParseErrorMessage); \
			bParamExists = true; \
			bSuccess = false; \
		} \
		if (!bParamExists) \
		{ \
			UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("Paired {0} and {1} are required for {2} mode."), { TEXT(#Key), TEXT(#Val), MODE_NAME })); \
			bSuccess = false; \
		} \
	}

#define BPT_ARG_OPTIONAL_SWITCH_PAIRS(Key, Val, Map) \
	{ \
		FString ParseErrorMessage; \
		bParamExists = ParsePairs(TEXT(#Key "="), TEXT(#Val "="), Switches, Config.Map, ParseErrorMessage); \
		if (!ParseErrorMessage.IsEmpty()) \
		{ \
			UE_LOGF(LogBuildPatchTool, Error, "%ls", *ParseErrorMessage); \
			bParamExists = true; \
			bSuccess = false; \
		} \
	}

#define BPT_ARG_REQUIRED_SWITCH_NOEMPTY_ALIAS(Switch, VarName) \
	bParamExists = ParseSwitch(TEXT(#Switch "="), Config.VarName, Switches); \
	if (!bParamExists) \
	{ \
		UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("{0} is required for {1} mode."), { TEXT(#Switch), MODE_NAME })); \
		bSuccess = false; \
	} \
	else if (FCmdLineProcessor::IsEmpty(Config.VarName)) \
	{ \
		UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("{0} should not be provided empty."), { TEXT(#Switch) })); \
		bSuccess = false; \
	}

#define BPT_ARG_REQUIRED_SWITCH_NOEMPTY(Switch) BPT_ARG_REQUIRED_SWITCH_NOEMPTY_ALIAS(Switch, Switch)

#define BPT_ARG_REQUIRED_FILENAME(Switch) \
	bParamExists = ParseSwitch(TEXT(#Switch "="), Config.Switch, Switches); \
	if (!bParamExists) \
	{ \
		UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("{0} is required for {1} mode."), { TEXT(#Switch), MODE_NAME })); \
		bSuccess = false; \
	} \
	else \
	{ \
		Config.Switch.TrimStartAndEndInline(); \
		if (FCmdLineProcessor::IsEmpty(Config.Switch)) \
		{ \
			UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("{0} should not be provided empty."), { TEXT(#Switch) })); \
			bSuccess = false; \
		} \
		NormalizeUriFile(Config.Switch); \
		if (Config.Switch.EndsWith(TEXT("/"))) \
		{ \
			UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("{0} should be a path to a file, not a directory."), { TEXT(#Switch) })); \
			bSuccess = false; \
		} \
	}

#define BPT_ARG_REQUIRED_FILENAME_EXISTS(Switch) \
	BPT_ARG_REQUIRED_FILENAME(Switch) \
	if (bParamExists && !FPaths::FileExists(Config.Switch)) \
	{ \
		UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("{0} does not exist. [{1}]"), { TEXT(#Switch), *Config.Switch })); \
		bSuccess = false; \
	}

#define BPT_ARG_OPTIONAL_FILENAME(Switch) \
	bParamExists = ParseSwitch(TEXT(#Switch "="), Config.Switch, Switches); \
	if (bParamExists && !FCmdLineProcessor::IsEmpty(Config.Switch)) \
	{ \
		if (Config.Switch.EndsWith(TEXT("/"))) \
		{ \
			UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("{0} should be a path to a file, not a directory."), { TEXT(#Switch) })); \
			bSuccess = false; \
		} \
		else \
		{ \
			NormalizeUriFile(Config.Switch); \
		} \
	}
	
#define BPT_ARG_OPTIONAL_FILENAME_EXISTS(Switch) \
	BPT_ARG_OPTIONAL_FILENAME(Switch) \
	if (bParamExists && !FPaths::FileExists(Config.Switch)) \
	{ \
		UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("{0} does not exist. [{1}]"), { TEXT(#Switch), *Config.Switch })); \
		bSuccess = false; \
	}

#define BPT_ARG_REQUIRED_PATH(Switch) \
	bParamExists = ParseSwitch(TEXT(#Switch "="), Config.Switch, Switches); \
	if (!bParamExists) \
	{ \
		UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("{0} is required for {1} mode."), { TEXT(#Switch), MODE_NAME })); \
		bSuccess = false; \
	} \
	else \
	{ \
		Config.Switch.TrimStartAndEndInline(); \
		if (FCmdLineProcessor::IsEmpty(Config.Switch)) \
		{ \
			UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("{0} should not be provided empty."), { TEXT(#Switch) })); \
			bSuccess = false; \
		} \
		else \
		{ \
			NormalizeUriPath(Config.Switch); \
			Config.Switch = FPaths::ConvertRelativePathToFull(Config.Switch); \
		}\
	}

#define BPT_ARG_REQUIRED_PATH_NOEMPTY(Switch) \
	BPT_ARG_REQUIRED_PATH(Switch) \
	if (bParamExists && !FPaths::DirectoryExists(Config.Switch)) \
	{ \
		UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("{0} does not exist. [{1}]"), { TEXT(#Switch), *Config.Switch })); \
		bSuccess = false; \
	} \
	if (bParamExists && IsEmptyDirectory(Config.Switch)) \
	{ \
		UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("{0} must not be an empty directory or a directory tree containing only empty files/directories. [{1}]"), { TEXT(#Switch), *Config.Switch })); \
		bSuccess = false; \
	}

#define BPT_ARG_REQUIRED_SWITCH_MAXSIZE(Switch, MaxSize) \
	bParamExists = ParseSwitch(TEXT(#Switch "="), Config.Switch, Switches); \
	if (!bParamExists) \
	{ \
		UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("{0} is required for {1} mode."), { TEXT(#Switch), MODE_NAME })); \
		bSuccess = false; \
	} \
	else if (FCmdLineProcessor::IsEmpty(Config.Switch)) \
	{ \
		UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("{0} should not be provided empty."), { TEXT(#Switch) })); \
		bSuccess = false; \
	} \
	else if (FCmdLineProcessor::ExceedsSize(Config.Switch, MaxSize)) \
	{ \
		UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("{0} should not exceed length {1}."), { TEXT(#Switch), MaxSize })); \
		bSuccess = false; \
	}
	
#define BPT_ARG_UNEXPECTED_SWITCH(Switch) \
	if (Switches.Contains(TEXT(#Switch))) \
	{ \
		UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("{0} should not be used with {1} mode."), { TEXT(#Switch), MODE_NAME })); \
		bSuccess = false; \
	}

#define BPT_ARG_OPTIONAL_SWITCH(Switch) \
	bParamExists = ParseSwitch(TEXT(#Switch "="), Config.Switch, Switches);

#define BPT_ARG_OPTIONAL_SWITCH_INT32(Switch) \
	bParamExists = ParseSwitch<int32>(TEXT(#Switch "="), Config.Switch, Switches);

#define BPT_ARG_OPTIONAL_SWITCH_NOEMPTY(Switch) \
	bParamExists = ParseSwitch(TEXT(#Switch "="), Config.Switch, Switches); \
	if (bParamExists && FCmdLineProcessor::IsEmpty(Config.Switch)) \
	{ \
		UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("Optional arg {0} should not be provided empty. Remove if not needed."), { TEXT(#Switch) })); \
		bSuccess = false; \
	}

#define BPT_ARG_OPTIONAL_SWITCH_CLAMPED(Switch, TInt, Min, Max) \
	{ \
		TInt Requested; \
		bParamExists = ParseSwitch(TEXT(#Switch "="), Requested, Switches); \
		if (bParamExists) \
		{ \
			Config.Switch = FMath::Clamp<TInt>(Requested, Min, Max); \
			if (Requested != Config.Switch) \
			{ \
				UE_LOGF(LogBuildPatchTool, Warning, "%ls", *FString::Format(TEXT("Requested -{0}={1} is outside of allowed range {2} >= n >= {3}. Please update your arg to be within range. Continuing with {4}."), { TEXT(#Switch), Requested, Max, Min, Config.Switch })); \
			} \
		} \
	}

#define BPT_ARG_OPTIONAL_SWITCH_MULTI(Switch) \
	bParamExists = ParseSwitches(TEXT(#Switch "="), Config.Switch##s, Switches);

#define BPT_ARG_OPTION(Option) \
	Config.b##Option = ParseOption(TEXT(#Option), Switches);

#define BPT_ARG_OPTION_MEMBER(Option) \
	b##Option = ParseOption(TEXT(#Option), Switches);

#define BPT_ARG_FEATURELEVEL_OPTIONAL_ALIAS(Switch) \
	{ \
		FString FeatureLevelString; \
		bParamExists = ParseSwitch(TEXT(#Switch"="), FeatureLevelString, Switches); \
		if (bParamExists) \
		{ \
			FeatureLevelString.TrimStartAndEndInline(); \
			BuildPatchServices::EFeatureLevel FeatureLevel; \
			LexFromString(FeatureLevel, *FeatureLevelString); \
			if (BuildPatchServices::EFeatureLevel::Invalid == FeatureLevel) \
			{ \
				UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("{0} FeatureLevel was provided, but {0} is not a recognized value."), { *FeatureLevelString })); \
				bSuccess = false; \
			} \
			else \
			{ \
				Config.Switch = FeatureLevel; \
			} \
		} \
	}
#define BPT_ARG_FEATURELEVEL_OPTIONAL BPT_ARG_FEATURELEVEL_OPTIONAL_ALIAS(FeatureLevel) \

#define BPT_ARG_FEATURELEVEL_REQUIRED \
	{ \
		FString FeatureLevelString; \
		bParamExists = ParseSwitch(TEXT(#Switch"="), FeatureLevelString, Switches); \
		if (bParamExists) \
		{ \
			FeatureLevelString.TrimStartAndEndInline(); \
			BuildPatchServices::EFeatureLevel FeatureLevel; \
			LexFromString(FeatureLevel, *FeatureLevelString); \
			if (FCmdLineProcessor::IsEmpty(FeatureLevelString)) \
			{ \
				UE_LOGF(LogBuildPatchTool, Error, "FeatureLevel should not be provided empty."); \
				bSuccess = false; \
			} \
			else if (BuildPatchServices::EFeatureLevel::Invalid == FeatureLevel) \
			{ \
				UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("{0} FeatureLevel was provided, but {0} is not a recognized value."), { *FeatureLevelString })); \
				bSuccess = false; \
			} \
			else \
			{ \
				Config.FeatureLevel = FeatureLevel; \
			} \
		} \
		else \
		{ \
			UE_LOGF(LogBuildPatchTool, Error, "%ls", *FString::Format(TEXT("FeatureLevel is required for {1} mode."), { TEXT(#Switch), MODE_NAME })); \
			bSuccess = false; \
		} \
	}

#define BPT_ARG_DIFFABORTTHRESHOLD_OPTIONAL \
	Config.DiffAbortThreshold = BuildPatchTool::FDiffAbortThresholdArgument::Parse(*this, Switches);

#define BPT_COMMANDLINE_FAIL_IF_TCHAR(Condition, ErrorStr) \
	if (Condition) \
	{ \
		UE_LOGF(LogBuildPatchTool, Error, "%ls", ErrorStr); \
		bSuccess = false; \
	}
#define BPT_COMMANDLINE_FAIL_IF(Condition, ErrorStr) \
	BPT_COMMANDLINE_FAIL_IF_TCHAR(Condition, TEXT(ErrorStr))

#define BPT_COMMANDLINE_END \
	return bSuccess; \
}

class FCmdLineProcessor
{
public:
	FCmdLineProcessor(int32 ArgC, TCHAR* ArgV[], const FString& WorkingDirectory, TFunction<void(const FString&)> ErrorLogger);

	bool ContainsFlag(const TCHAR* Flag) const;

	bool ContainsSwitch(const TCHAR* Switch) const;

	bool GetSwitchValue(const TCHAR* Switch, FString& OutValue) const;

	const FString& GetOriginalCommandLine();

	FString GetEngineString();

	void ProcessDragAndDrop();

	bool ProcessCommandlineFiles();

	bool HandleLegacyCommandline();

	bool ConvertCommandlinePaths();

	void AddDesiredEngineFlags();

	// Helpers for BPT_MODE_COMMANDLINE macros.
	static inline bool IsEmpty(const FString& Val) { return Val.IsEmpty(); }
	static inline bool ExceedsSize(const FString& Val, int32 Max) { return Val.Len() > Max; }

private:
	const FString WorkingDirectory;
	TFunction<void(const FString&)> ErrorLogger;
	FString OriginalCommandLine;
	TArray<FString> Tokens;

	const TArray<FString> PathParamsToMakeAbsolute = {
		TEXT("-CloudDir="), TEXT("-ManifestA="), TEXT("-ManifestB="), TEXT("-ManifestC="), TEXT("-DownloadCloudDirectory="), TEXT("-ResultDataFile="),
		TEXT("-BuildRoot="), TEXT("-CloudDirs="), TEXT("-OutputFile="), TEXT("-File="), TEXT("-MergeFileList="), TEXT("-PrevManifestFile="),
		TEXT("-ManifestFile="), TEXT("-SearchPath="), TEXT("-InputFile="), TEXT("-Manifest="), TEXT("-OutputDir="),
		TEXT("-DeletedChunkLogFile="), TEXT("-PrevManifest=") };

	const TArray<FString> PathParamsToCollapseRelativeDirectories = {
		TEXT("-AppLaunch=") };
};
