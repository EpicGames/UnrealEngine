// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Interfaces/ToolMode.h"

namespace BuildPatchTool
{
	class FDiffAbortThresholdArgument
	{
	public:
		static TOptional<BuildPatchServices::FDiffAbortThreshold> Parse(BuildPatchTool::IToolMode& ToolMode, const TArray<FString>& Switches);
	private:
		static BuildPatchServices::FDiffAbortThreshold ClampToSaneRange(const FString& DiffAbortThresholdRaw, const BuildPatchServices::FDiffAbortThreshold& DiffAbortThreshold);
	};

}

