// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Interfaces/IBuildManifest.h"

enum class EMetadataOutputFormat
{
	Human = 0,
	Json,
	InvalidOrMax
};

bool LexFromString(EMetadataOutputFormat& OutValue, const TCHAR* Buffer);

class FMetadataSerialiser
{
public:
	static FString SerialiseMetadata(const IBuildManifest& Manifest, EMetadataOutputFormat OutputFormat);
};

