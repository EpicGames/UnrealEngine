// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Interfaces/IBuildPatchServicesModule.h"
#include "Algo/AllOf.h"
#include "HAL/FileManager.h"
#include "Misc/Optional.h"
#include "BuildPatchTool.h"

#define IMPLEMENT_BPT_MODE(ModeName, ModeClass) \
class ModeClass; \
TCHAR const * const ModeClass::MODE_NAME = TEXT(#ModeName); \
struct FAutoRegister##ModeName \
{ \
	FAutoRegister##ModeName() \
	{ \
		using namespace BuildPatchTool; \
		FToolModeFactory::GetModeRegistryDelegate().AddLambda([](TMap<FString, FToolModeFactory::FToolModeCreateFunc>& OutModes) \
			{ \
				OutModes.Add(TEXT(#ModeName), [](IBuildPatchServicesModule& BpsInterface, const TCHAR* CommandLine){ return new ModeClass(BpsInterface, CommandLine); }); \
			}); \
	} \
} AutoRegister##ModeName;

#define IMPLEMENT_BPT_MODE_ALIAS(ModeName, ModeClass) \
struct FAutoRegister##ModeName \
{ \
	FAutoRegister##ModeName() \
	{ \
		using namespace BuildPatchTool; \
		FToolModeFactory::GetModeRegistryDelegate().AddLambda([](TMap<FString, FToolModeFactory::FToolModeCreateFunc>& OutModes) \
			{ \
				OutModes.Add(TEXT(#ModeName), [](IBuildPatchServicesModule& BpsInterface, const TCHAR* CommandLine){ return new ModeClass(BpsInterface, CommandLine); }); \
			}); \
	} \
} AutoRegister##ModeName;

namespace Constants
{
	static const FString Comma(TEXT(","));
	static const FString Equals(TEXT("="));
	static const FString DoubleQuote(TEXT("\""));
	static const FString SingleQuote(TEXT("'"));
	static const FString Slash(TEXT("/"));
	static const FString Backslash(TEXT("\\"));
	static const FString Custom(TEXT("custom"));
	static const FString CustomInt(TEXT("customint"));
	static const FString CustomFloat(TEXT("customfloat"));
}

// Detect if a template parameter is a TSet
template<typename T>
struct is_ue_set : std::false_type {};

template<typename... T>
struct is_ue_set<TSet<T...>> : std::true_type {};

// Detect if template parameter is a TArray
template<typename T>
struct is_ue_array : std::false_type {};

template<typename... T>
struct is_ue_array<TArray<T...>> : std::true_type {};

namespace BuildPatchTool
{
	enum class EReturnCode : int32;

	class IToolMode
	{
	public:
		TCHAR const * const EqualsStr = TEXT("=");
		TCHAR const * const QuoteStr = TEXT("\"");

		struct FInstallActionArgs
		{
		public:
			TOptional<FString> PrevManifestUri;
			TOptional<FString> NewManifestUri;
			TOptional<TSet<FString>> InstallTags;
		};

	public:
		virtual ~IToolMode() { }

		virtual EReturnCode Execute() = 0;

		bool bHelp = false;
		template<typename T>
		void PrintHelp()
		{
			for (int32 i = 0; i < T::NumLines; i++)
			{
				UE_LOGF(LogBuildPatchTool, Display, "%ls", T::Text[i]);
			}
		}

		/**
		 * Helper for normalizing a URI referring to a file. We make sure not to cause issues with URIs that separate a protocol and authority with ://
		 * and windows UNC paths that begin with \\. Windows UNC paths still work in UE4 code with //.
		 *
		 * @param UriFile       The file URI to be Normalized.
		 */
		void NormalizeUriFile(FString& UriFile)
		{
			// Replace all slashes.
			UriFile.ReplaceInline(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive);
		}

		void NormalizeUriFileInKeys(TArray<TPair<FString, FString>>& Pairs)
		{
			for (TPair<FString, FString>& Pair : Pairs)
			{
				NormalizeUriFile(Pair.Key);
			}
		}

		/**
		 * Helper for normalizing a URI referring to a path. We make sure not to cause issues with URIs that separate a protocol and authority with ://
		 * and windows UNC paths that begin with \\. Windows UNC paths still work in UE4 code with //.
		 *
		 * @param UriPath       The path URI to be Normalized.
		 */
		void NormalizeUriPath(FString& UriPath)
		{
			// Fix all slashes.
			NormalizeUriFile(UriPath);
			// Remove trailing slashes, unless it is for a windows based drive (e.g. "C:/")
			int32 TrailingSlashIdx = UriPath.Len() - 1;
			while (UriPath.EndsWith(TEXT("/"), ESearchCase::CaseSensitive) && !UriPath.EndsWith(TEXT(":/"), ESearchCase::CaseSensitive))
			{
				// Overwrite trailing slash with terminator, and trim size.
				UriPath.GetCharArray()[TrailingSlashIdx--] = 0;
				UriPath.GetCharArray().SetNum(UriPath.GetCharArray().Num() - 1, EAllowShrinking::No);
			}
		}

		bool IsEmptyDirectory(const FString& Directory)
		{
			IFileManager& FM = IFileManager::Get();
			TArray<FString> FoundFiles;
			FM.FindFilesRecursive(FoundFiles, *Directory, TEXT("*"), true, false);
			for (const FString& File : FoundFiles)
			{
				if (FM.FileSize(*File) > 0)
				{
					return false;
				}
			}
			return true;
		}

		/**
		 * Helper for parsing a switch from an array of switches, usually produced using FCommandLine::Parse(..)
		 *
		 * @param InSwitch      The switch name, ending with =. E.g. option=, foo=. It would usually be a compile time const.
		 * @param Value         Receives the value from the switch.
		 * @param Switches      The array of switches to search through.
		 * @return true if the switch was found.
		 */
		template <typename TValueType>
		bool ParseSwitch(const TCHAR* InSwitch, TValueType& Value, const TArray<FString>& Switches)
		{
			// Debug check requirements for InSwitch
			checkSlow(InSwitch != nullptr);
			checkSlow(InSwitch[FCString::Strlen(InSwitch)-1] == TEXT('='));

			if (const FString* Switch = Switches.FindByPredicate([InSwitch](const FString& Switch) { return Switch.StartsWith(InSwitch); }))
			{
				FString StringValue;
				Switch->Split(EqualsStr, nullptr, &StringValue);
				return ParseValue(StringValue, Value);
			}
			return false;
		}

		/**
		 * Helper for parsing an array of multiple same name switches from the full array of switches, usually produced using FCommandLine::Parse(..)
		 *
		 * @param InSwitch      The switch name, ending with =. E.g. option=, foo=. It would usually be a compile time const.
		 * @param Values        Receives the values from the switches.
		 * @param Switches      The array of switches to search through.
		 * @return true if at least one match was found.
		 */
		template <typename TValueType>
		bool ParseSwitches(const TCHAR* InSwitch, TArray<TValueType>& Values, const TArray<FString>& Switches)
		{
			// Debug check requirements for InSwitch
			checkSlow(InSwitch != nullptr);
			checkSlow(InSwitch[FCString::Strlen(InSwitch)-1] == TEXT('='));

			bool bFoundValue = false;
			for (const FString& Switch : Switches)
			{
				if (Switch.StartsWith(InSwitch))
				{
					FString StringValue;
					TValueType Value;
					Switch.Split(EqualsStr, nullptr, &StringValue);
					if (ParseValue(StringValue, Value))
					{
						Values.Emplace(MoveTemp(Value));
						bFoundValue = true;
					}
				}
			}
			return bFoundValue;
		}

		bool ParseOption(const TCHAR* InSwitch, const TArray<FString>& Switches)
		{
			return Switches.Contains(InSwitch);
		}

		template<typename TFirst, typename TSecond, typename TContainer>
		bool ParsePairs(const TCHAR* InFirstSwitch, const TCHAR* InSecondSwitch, const TArray<FString>& Switches, TContainer& OutPairs, FString& ErrorMessage)
		{
			bool bFoundAnyPair = false;
			bool bFoundFirst = false;
			bool bFoundSecond = false;
			TFirst FirstValue;
			TSecond SecondValue;
			for (int32 SwitchIndex = 0; SwitchIndex < Switches.Num(); SwitchIndex++)
			{
				const FString& InputSwitch = Switches[SwitchIndex];
				const bool bIsFirst = InputSwitch.StartsWith(InFirstSwitch);
				const bool bIsSecond = InputSwitch.StartsWith(InSecondSwitch);
				if (bIsFirst || bIsSecond)
				{
					 // Check we are alternating
					if ((bIsFirst && bFoundFirst)
					 || (bIsSecond && bFoundSecond))
					{
						// The error will be set in the check after this loop.
						break;
					}
					FString ValStr;
					InputSwitch.Split(IToolMode::EqualsStr, nullptr, &ValStr);
					if (bIsFirst && !ParseValue(ValStr, FirstValue))
					{
						ErrorMessage = FString::Printf(TEXT("Invalid parse of argument %s value: %s."),
							*FString(FCString::Strlen(InFirstSwitch) - 1, InFirstSwitch),
							*ValStr);
						return false;
					}
					else if (bIsSecond && !ParseValue(ValStr, SecondValue))
					{
						ErrorMessage = FString::Printf(TEXT("Invalid parse of argument %s value: %s."),
							*FString(FCString::Strlen(InSecondSwitch) - 1, InSecondSwitch),
							*ValStr);
						return false;
					}
					bFoundFirst |= bIsFirst;
					bFoundSecond |= bIsSecond;
				}
				if (bFoundFirst && bFoundSecond)
				{
					OutPairs.Emplace(MoveTemp(FirstValue), MoveTemp(SecondValue));
					bFoundAnyPair = true;
					bFoundFirst = false;
					bFoundSecond = false;
					FirstValue = TFirst();
					SecondValue = TSecond();
				}
			}
			if (bFoundFirst != bFoundSecond)
			{
				ErrorMessage = FString::Printf(TEXT("%s and %s parameters must be provided as pairs."),
					*FString(FCString::Strlen(InFirstSwitch) - 1, InFirstSwitch),
					*FString(FCString::Strlen(InSecondSwitch) - 1, InSecondSwitch));
				return false;
			}
			return bFoundAnyPair;
		}

		bool ParsePairs(const TCHAR* InFirstSwitch, const TCHAR* InSecondSwitch, const TArray<FString>& Switches, TMap<FGuid, TArray<uint8>>& OutPairs, FString& ErrorMessage)
		{
			return BuildPatchTool::IToolMode::ParsePairs<FGuid, TArray<uint8>, TMap<FGuid, TArray<uint8>>>(InFirstSwitch, InSecondSwitch, Switches, OutPairs, ErrorMessage);
		}

		bool ParsePairs(const TCHAR* InFirstSwitch, const TCHAR* InSecondSwitch, const TArray<FString>& Switches, TArray<TPair<FString, FString>>& OutPairs, FString& ErrorMessage)
		{
			return BuildPatchTool::IToolMode::ParsePairs<FString, FString, TArray<TPair<FString, FString>>>(InFirstSwitch, InSecondSwitch, Switches, OutPairs, ErrorMessage);
		}

		bool ParseInstallActions(const TCHAR* InPrevManifestSwitch, const TCHAR* InNewManifestSwitch, const TCHAR* InTagsSwitch, const TArray<FString>& Switches, TArray<FInstallActionArgs>& OutActions, FString& ErrorMessage)
		{
			TOptional<FString> PrevManifest;
			TOptional<FString> NewManifest;
			TOptional<FString> InstallTags;
			// Using some lambda functions to reduce copy-paste code noise.
			auto HasEmptyValue = [](const TOptional<FString>& Val) -> bool
				{
					return Val.IsSet() && Val.GetValue().IsEmpty();
				};
			auto HasNonEmptyValue = [](const TOptional<FString>& Val) -> bool
				{
					return Val.IsSet() && !Val.GetValue().IsEmpty();
				};
			auto PushAction = [&]()
				{
					// Each action requires at least one manifest path, if we enter this function having only parsed empty strings or install tags alone
					// then the commandline is not valid.
					if (!HasNonEmptyValue(PrevManifest) && !HasNonEmptyValue(NewManifest))
					{
						ErrorMessage = FString::Printf(TEXT("Invalid format for install action arguments. Check the %s, %s, and %s inputs are correct and properly ordered."),
							InPrevManifestSwitch,
							InNewManifestSwitch,
							InTagsSwitch);
					}
					// If provided, URIs should not be epty strings.
					else if (HasEmptyValue(PrevManifest))
					{
						ErrorMessage = FString::Printf(TEXT("%s should not be provided empty."),
							InPrevManifestSwitch);
					}
					else if (HasEmptyValue(NewManifest))
					{
						ErrorMessage = FString::Printf(TEXT("%s should not be provided empty."),
							InNewManifestSwitch);
					}
					else
					{
						TSet<FString> TagSet;
						// If tags were not provided, this is interpreted as empty set (resulting in all files).
						// An empty string for tags on commandline interprets as a set with just an empty string in it (resulting in untagged files only).
						if (InstallTags.IsSet())
						{
							ParseValue(InstallTags.GetValue(), TagSet); // Always returns true
							InstallTags.Reset();
						}
						OutActions.Add({ MoveTemp(PrevManifest), MoveTemp(NewManifest), MoveTemp(TagSet) });
						// MoveTemp above does not reset a TOptional<FString> state, it just leaves it set as empty string.
						PrevManifest.Reset();
						NewManifest.Reset();
						// InstallTags is reset above.
					}
				};
			auto TakeComponent = [&PushAction, this](const FString& InputString, TOptional<FString>& OutVal)
				{
					// If we already had a this switch, this is the start of a new action, so push the previous one.
					if (OutVal.IsSet())
					{
						PushAction();
					}
					OutVal = InputString;
				};
			// Loop through all of the switches to find each action triplet. Not all args are requred per action.
			// PushAction(...) handles making sure we got at least one manifest.
			for (int32 SwitchIndex = 0; SwitchIndex < Switches.Num() && ErrorMessage.IsEmpty(); SwitchIndex++)
			{
				// Grab and trim the switch value
				const FString& InputSwitch = Switches[SwitchIndex];
				FString ValStr;
				InputSwitch.Split(IToolMode::EqualsStr, nullptr, &ValStr);
				ValStr = ValStr.TrimQuotes();

				// Check for PrevManifest
				if (InputSwitch.StartsWith(InPrevManifestSwitch))
				{
					NormalizeUriFile(ValStr);
					TakeComponent(ValStr, PrevManifest);
				}
				// Check for NewManifest
				else if (InputSwitch.StartsWith(InNewManifestSwitch))
				{
					NormalizeUriFile(ValStr);
					TakeComponent(ValStr, NewManifest);
				}
				// Check for InstallTags
				else if (InputSwitch.StartsWith(InTagsSwitch))
				{
					TakeComponent(ValStr, InstallTags);
				}
			}
			// Check for the last action that was not pushed during the loop.
			if (PrevManifest.IsSet() || NewManifest.IsSet() || InstallTags.IsSet())
			{
				PushAction();
			}
			// At lease one action is required to be successful.
			if (ErrorMessage.IsEmpty() && OutActions.Num() == 0)
			{
				ErrorMessage = FString::Printf(TEXT("At least one install action is required. Check you have provided the %s, %s, and %s arguments necessary."),
					InPrevManifestSwitch,
					InNewManifestSwitch,
					InTagsSwitch);
			}
			return ErrorMessage.IsEmpty();
		}

		// Helper to run ParseInstallActions with default argument names.
		bool ParseInstallActions(const TArray<FString>& Switches, TArray<FInstallActionArgs>& OutActions, FString& ErrorMessage)
		{
			return ParseInstallActions(TEXT("PrevManifest="), TEXT("Manifest="), TEXT("InstallTags="), Switches, OutActions, ErrorMessage);
		}

		bool ParseValue(const FString& ValueIn, FString& ValueOut)
		{
			ValueOut = ValueIn.TrimQuotes();
			return true;
		}

		bool ParseValue(const FString& ValueIn, uint64& ValueOut)
		{
			if (FCString::IsNumeric(*ValueIn) && !ValueIn.Contains(TEXT("-"), ESearchCase::CaseSensitive))
			{
				ValueOut = FCString::Strtoui64(*ValueIn, nullptr, 10);
				return true;
			}
			return false;
		}

		bool ParseValue(const FString& ValueIn, uint32& ValueOut)
		{
			if (FCString::IsNumeric(*ValueIn) && !ValueIn.Contains(TEXT("-"), ESearchCase::CaseSensitive))
			{
				ValueOut = (uint32)FCString::Strtoi(*ValueIn, nullptr, 10);
				return true;
			}
			return false;
		}

		bool ParseValue(const FString& ValueIn, int32& ValueOut)
		{
			if (FCString::IsNumeric(*ValueIn) && !ValueIn.Contains(TEXT("-"), ESearchCase::CaseSensitive))
			{
				ValueOut = FCString::Strtoi(*ValueIn, nullptr, 10);
				return true;
			}
			return false;
		}

		bool ParseValue(const FString& ValueIn, FGuid& ValueOut)
		{
			return FGuid::Parse(ValueIn, ValueOut);
		}

		template<typename TCollection>
		typename std::enable_if<is_ue_set<TCollection>::value || is_ue_array<TCollection>::value, bool>::type
			ParseValue(const FString& ValueIn, TCollection& ValueOut)
		{
			TArray<FString> TempArray;
			ValueOut.Empty();
			FString CleanValueIn = ValueIn.TrimQuotes();
			if (CleanValueIn.ParseIntoArray(TempArray, TEXT(","), false) != 0)
			{
				for (FString& TempElem : TempArray)
				{
					TempElem.TrimStartAndEndInline();
					ValueOut.Add(MoveTemp(TempElem));
				}
			}
			else
			{
				// Add a single empty element. This is what is intended if switch is present on the command
				// line and contains no commas.
				ValueOut.Add(TEXT(""));
			}
			return true;
		}

		bool ParseValue(const FString& ValueIn, TArray<uint8>& ValueOut)
		{
			// Here we are expecting a string into bytes, and for now, we are supporting Hex only.
			const int32 NumChars = ValueIn.Len();
			// An odd number of characters means the first byte is a padded nibble
			int32 NumBytes = ValueIn.Len() / 2;
			if (NumChars % 2 == 1)
			{
				++NumBytes;
			}
			ValueOut.Empty(NumBytes);
			ValueOut.AddZeroed(NumBytes);
			return ValueOut.Num() == 0 || HexToBytes(ValueIn, &ValueOut[0]) == NumBytes;
		}

		template<typename TEnum>
		typename std::enable_if<std::is_enum<TEnum>::value, bool>::type
			ParseValue(const FString& ValueIn, TEnum& ValueOut)
		{
			LexFromString(ValueOut, *ValueIn);
			return ValueOut != TEnum::InvalidOrMax;
		}
	};

	class FToolModeFactory
	{
	public:
		typedef TFunction<IToolMode* (IBuildPatchServicesModule&, const TCHAR* /*CommandLine*/)> FToolModeCreateFunc;
		typedef TMulticastDelegate<void(TMap<FString, FToolModeCreateFunc>&)> FModeRegistryDelegate;

		static IToolMode* Create(IBuildPatchServicesModule& BpsInterface, const TCHAR* CommandLine);
		static FModeRegistryDelegate& GetModeRegistryDelegate();
	};
}
